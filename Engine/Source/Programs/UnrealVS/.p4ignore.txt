**/bin/Release
**/bin/Debug
*.csproj.user