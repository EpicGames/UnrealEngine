If nuget packages are updated, the minimum Visual Studio version needs to be updated to match in the .vsixmanifest.

Ex. Microsoft.VisualStudio.SDK 16.9.31025.194 = Minimum Visual Studio version needs to be 16.9

# Installation

In Visual Studio
Tool->Options->Environment->Extension
Add an Addition Extension Gallery.
Point the URL to the AtomFeed of where UnrealVS.vsix is synced
ex.

Name: UnrealVS
URL: file:///D:/dev/ue5main/Engine/Extras/UnrealVS/AtomFeed.xml

# Debugging
In order to debug UnrealVS.vsix:
- Set the Startup project to UnrealVS.2022
- Update the Debug properties for the UnrealVS.2022 csproj
    - Start external program set to: C:\Program Files\Microsoft Visual Studio\2022\Professional\Common7\IDE\devenv.exe
    - Command line arguments set to: /rootsuffix Exp