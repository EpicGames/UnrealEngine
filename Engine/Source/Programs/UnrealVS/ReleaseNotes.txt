1.87.0.0
* Fixed command line combo box not updating after switching entries in VS2026.

1.86.0.0
* Added Cycle Related File command. This command will cycle through files related to the current file. For example, if you are in a .cpp file, it will open the corresponding .h file, and vice versa.

1.85.0.0
* Added option to not close the File Browser window after a file is select.
* Added the module and file path to File Browser.
* Added a context menu to File Browser.

1.84.9.6
* Usability improvements for startup project selector

1.84.9.5
* Adding "Copy Unreal #include Statement" to the document tab context menu.

1.84.9.4
* Adding UnrealVS.UBT options to build Modules from Solution Explorer folders.

1.84.9.3
* Addressing situations where "Compile and Profile Single File" didn't work correctly with Compile Score.
* Disabling "Compile and Profile Single File" when Compile Score is not installed.
* Adding "UnrealVS.UBT->Build and Profile" project context menu option that uses Compile Score.

1.84.9.2
* Fix issue where UbaVisualizer will not work on projects outside of Engine hierarchy.

1.84.9.1
* Fix issue where running UnrealVS.P4->History would occasionally spawn an empty p4 history window.

1.84.9.0
* Display errors during test discovery in the Tests output log.
* Don't run AutoSDK when executing tests from the Test Explorer.
* Fix hang when debugging multiple tests from the Test Explorer.

1.84.8.4
* Fix a potential crash in the UbaVisualizer on start up

1.84.8.3
* Bug fix where invoking p4v commands could cause visual studio to hang

1.84.8.2
* Bug fix where the package info string is incorrect

1.84.8.1
* Bug fix for UbaVisualizer throwing exception when opening crash dumps

1.84.8.0
* Fix dpi scaling support for UBA

1.84.7.0
* Fix exception for older VS versions

1.84.7.0
* Update NuGet packages
* Remove VS2019 support

1.84.5.0
* Added UbaVisualizer panel. If open solution is under UE engine path this panel will launch UbaVisualizer.exe embedded and listen to uba traces spawned by vs instance

1.84.4.0
* Fixed a race condition at initialization time that would make Visual Studio unresponsive.

1.84.3.9
* Fixed issues with compile single file with profile. Did not work if selected project was a c# project. Should now also work with non-ue projects

1.84.3.8
* Fixed issues with context menu items.

1.84.3.7
* Prefix encrypted Perforce server port address with "ssl:"

1.84.3.6
* Fix P4 checkout queue causing hang if p4.exe is not found in path.

1.84.3.5
* Fix null reference exception when setting command line for C# project without a Properties directory

1.84.3.4
* Bug fix for incorrect paths when compiling a single file
* Open .asm after generating assembly
* Disable remaining compile single buttons if a valid file isn't selected
* Bug fix to show additional .target.cs files

1.84.3.3
* Bug fix for low level test projects - do not auto prefix project path in command line

1.84.3.2
* Bug fix for compiling a single file when us

1.84.3.1
* Bug fix for leaving in debugging code

1.84.3
* Added command to generate assembly file while compiling single file

1.84.2
* Fix to Unreal.TestAdapater to output test results to a file

1.84.1
* P4 Commands now are more aggressive about finding your working directory

1.83
* Open for Edit now also Adds if the file is not already added
* added Revert command
* the File History command no longer hangs
* the Timelapse command now scrolls to the current line

1.80
* updated Unreal.TestAdapater to report duration's
* added a menu for extension to improve discover-ability of features
* added option to use P4V diff instead of Visual Studio diff
* refactored code to share files between VS2019 and VS2022 versions
* chanaged the File Browser to handle paste of file names

1.79
* added Unreal.TestAdapter to run low level tests from the text explorer

1.78
* fixed race condition when starting async checkout

1.77
* added File Browser dialog (open with command UnrealVs.FileBrowser)
* added so Visual Studio title/solution show folder name of UE5.sln when UE5.sln is opened (no other solutions)

1.76
* files scheduled for async checkout are now saved in a .p4checkout.txt file in the solution directory, to allow recovery if Perforce is down or VS terminates.
* fixed some launch settings being deleted when using command line editor.

1.72
* fixed exception when opening project missing a referenced configuration
* added exceptions to guard against code UI code not being accessed on the UI thread

1.71
* add p4FastReconcile
* add ability to halt p4 output auto focus

1.70
* add option to allow async checkout
* resolved concurrent command issues
* fix issue with P4V commands waiting for exit
* resolve issue where Checkout on Edit was not honoring the user option
