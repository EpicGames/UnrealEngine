#Things to know

* dll must be called XXX.TestAdapter.dll. VS looks for dll's with this pattern.

* source.extension.vsixmanifest has a reference to this project. that does the deployment of the dll. 
   Under Assets with a type of `UnitTestExtension`

* TestDiscoverer relies on a file called `(exe).is_unreal_test existing`.
   The TestTargetRules currently outputs this file
   The reason for this is to quickly identify text executables. 
   Another option would be add a special string into the exe and then look for it in the binary

#Debugging

Easiest way to debug the TestDisoverer or the TestExecutor by adding `System.Diagnostics.Debugger.Launch();` to 
`DiscoverTests` or `RunTests` respectively. 
