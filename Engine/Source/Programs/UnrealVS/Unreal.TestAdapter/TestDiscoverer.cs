// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Text;
using System.Xml;
using Microsoft.VisualStudio.TestPlatform.ObjectModel;
using Microsoft.VisualStudio.TestPlatform.ObjectModel.Adapter;
using Microsoft.VisualStudio.TestPlatform.ObjectModel.Logging;

namespace Unreal.TestAdapter
{
	[FileExtension(".exe")]
	[DefaultExecutorUri("executor://UnrealTestExecutor")]
	class TestDiscoverer : ITestDiscoverer
	{
		static readonly Uri ExecutorUri = new Uri("executor://UnrealTestExecutor");
		internal static readonly TestProperty DisabledTestProperty = TestProperty.Register("TestDiscoverer.DisabledTestProperty", "DisabledTestProperty", string.Empty, string.Empty, typeof(bool), (object o) => o is bool, TestPropertyAttributes.Hidden, typeof(TestDiscoverer));

		public void DiscoverTests(IEnumerable<string> sources, IDiscoveryContext discoveryContext, IMessageLogger logger, ITestCaseDiscoverySink discoverySink)
		{
			foreach (var source in sources)
			{
				//cheap way to look for tests. outputted by the TestTargetRules
				if (!File.Exists(source + ".is_unreal_test"))
					continue;

				string xml = GetTestCaseXml(source, logger);
				ParseXml(xml, discoverySink, source);
			}
		}

		string GetTestCaseXml(string source, IMessageLogger logger)
		{
			Process process = new Process();
			process.StartInfo.FileName = source;
			process.StartInfo.UseShellExecute = false;
			process.StartInfo.RedirectStandardOutput = true;
			process.StartInfo.CreateNoWindow = true;
			process.StartInfo.Arguments = "--list-tests * --reporter xml";
			process.StartInfo.RedirectStandardError = true;

			StringBuilder error = new StringBuilder();
			process.ErrorDataReceived += new DataReceivedEventHandler((sender, e) =>
			{
				if (e.Data != null)
				{
					error.AppendFormat("\n\t{0}", e.Data);
				}
			});

			process.Start();
			process.BeginErrorReadLine();
			string result = process.StandardOutput.ReadToEnd();
			bool exited = process.WaitForExit(500);

			if (error.Length > 0)
			{
				string message = string.Format("Test source \"{0}\" produced errors:\n{{{1}\n}}", source, error);
				logger.SendMessage(TestMessageLevel.Error, message);
			}

			if (exited)
			{
				if (process.ExitCode != 0)
				{
					string message = string.Format("Test source \"{0}\" exited with code {1}.", source, process.ExitCode);
					logger.SendMessage(TestMessageLevel.Error, message);
				}
			}
			else
			{
				string message = string.Format("Test source \"{0}\" failed to exit within a reasonable time.", source, process.ExitCode);
				logger.SendMessage(TestMessageLevel.Error, message);
			}

			return result;
		}

		static void ParseXml(string xml, ITestCaseDiscoverySink discoverySink, string source)
		{
			TextReader xmlText = new StringReader(xml);
			using (var xmlReader = XmlReader.Create(xmlText, new XmlReaderSettings() { IgnoreWhitespace = true }))
			{
				while (xmlReader.Read())
				{
					switch (xmlReader.NodeType)
					{
						case XmlNodeType.Element:
							if (xmlReader.Name == "TestCase" && !xmlReader.IsEmptyElement)
							{
								if (ParseTestCase(xmlReader, source, out TestCase testCase))
								{
									discoverySink.SendTestCase(testCase);
								}
							}
							break;
						case XmlNodeType.EndElement:
							if (xmlReader.Name == "MatchingTests")
							{
								//quit because sometimes there is extra lines are the xml
								//ie. Shutting down and abandoning module CoreUObject (4)
								return;
							}
							break;
						default:
							break;
					}
				}
			}
		}
		static bool ParseTestCase(XmlReader xmlReader, string source, out TestCase testCase)
		{
			testCase = null;
			//advance past the start element
			xmlReader.Read();

			string fullyQualifiedName = null;
			string codeFilePath = null;
			int lineNumber = 0;
			bool disabled = false;
			while (xmlReader.NodeType != XmlNodeType.EndElement)
			{
				if (xmlReader.IsEmptyElement)
				{
					xmlReader.Skip();
				}
				else if (xmlReader.Name == "Name")
				{
					fullyQualifiedName = xmlReader.ReadElementContentAsString();
				}
				else if (xmlReader.Name == "SourceInfo")
				{
					xmlReader.Read();
					while (xmlReader.NodeType != XmlNodeType.EndElement)
					{
						if (xmlReader.Name == "File")
						{
							codeFilePath = xmlReader.ReadElementContentAsString();
						}
						else if (xmlReader.Name == "Line")
						{
							lineNumber = xmlReader.ReadElementContentAsInt();
						}
						else
						{
							xmlReader.Read();
						}
					}
					xmlReader.Read(); //read </SourceInfo>
				}
				else if(xmlReader.Name == "Tags")
				{
					string tags = xmlReader.ReadElementContentAsString();
					disabled = tags.Contains("[.]");
				}
				else
				{
					xmlReader.Skip(); //skip elements we don't care about
				}
			}
			if (codeFilePath != null && fullyQualifiedName != null)
			{
				testCase = new TestCase(fullyQualifiedName, ExecutorUri, source) { LineNumber = lineNumber, CodeFilePath = codeFilePath };
				testCase.SetPropertyValue(DisabledTestProperty, disabled);
			}
			return testCase != null;
		}
	}

	
}
