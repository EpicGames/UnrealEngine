// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Threading.Tasks;
using System.Xml;
using Microsoft.VisualStudio.TestPlatform.ObjectModel;
using Microsoft.VisualStudio.TestPlatform.ObjectModel.Adapter;

namespace Unreal.TestAdapter
{
	[ExtensionUri("executor://UnrealTestExecutor")]
	class TestExecutor : ITestExecutor
	{
		bool WasCancelled = false;

		public void Cancel()
		{
			WasCancelled = true;
		}

		public void RunTests(IEnumerable<string> sources, IRunContext runContext, IFrameworkHandle frameworkHandle)
		{
		}

		public void RunTests(IEnumerable<TestCase> tests, IRunContext runContext, IFrameworkHandle frameworkHandle)
		{
			WasCancelled = false;

			if (runContext.IsBeingDebugged)
			{
				foreach (TestCase testCase in tests)
				{
					RunTestCase(testCase, frameworkHandle, true);
				}
			}
			else
			{
				Parallel.ForEach(tests, testCase =>
				{
					RunTestCase(testCase, frameworkHandle, false);
				});
			}
		}

		void RunTestCase(TestCase testCase, IFrameworkHandle frameworkHandle, bool debug)
		{
			if (WasCancelled)
			{
				return;
			}

			if (testCase.GetPropertyValue(TestDiscoverer.DisabledTestProperty, false))
			{
				frameworkHandle.RecordResult(new TestResult(testCase) { Outcome = TestOutcome.Skipped });
				return;
			}

			string tempFile = Path.Combine(Path.GetTempPath(), Path.GetTempFileName());
			string args = string.Format("\"{0}\" --reporter unreal -o {1} --extra-args -Multiprocess", testCase.FullyQualifiedName, tempFile);
			frameworkHandle.RecordStart(testCase);

			if (debug)
			{
				// It does not appear to be safe to call LaunchProcessWithDebuggerAttached in parallel
				string workingDir = Path.GetDirectoryName(testCase.Source);
				int processId = frameworkHandle.LaunchProcessWithDebuggerAttached(testCase.Source, workingDir, args, null);
				Process process = Process.GetProcessById(processId);
				process.WaitForExit();
			}
			else
			{
				using (Process process = new Process())
				{
					process.StartInfo.FileName = testCase.Source;
					process.StartInfo.Arguments = args;
					process.StartInfo.UseShellExecute = false;
					process.Start();
					process.WaitForExit();
				}
			}

			string xmlString = null;
			if (File.Exists(tempFile))
			{
				xmlString = File.ReadAllText(tempFile);
				File.Delete(tempFile);
			}

			var testResult = new TestResult(testCase) { Outcome = TestOutcome.NotFound };
			ParseResult(xmlString, testResult);
			frameworkHandle.RecordEnd(testCase, testResult.Outcome);
			frameworkHandle.RecordResult(testResult);
		}

		void ParseResult(string xmlString, TestResult testResult)
		{
			TextReader textReader = new StringReader(xmlString);
			using (XmlReader xml = XmlReader.Create(textReader, new XmlReaderSettings() { IgnoreWhitespace = true }))
			{
				while (xml.Read())
				{
					switch (xml.NodeType)
					{
						case XmlNodeType.Element:
							if (xml.Name == "testcase" && xml.GetAttribute("name") == testResult.TestCase.FullyQualifiedName)
							{
								ParseTestCaseResult(xml, testResult);
							}
							break;
						case XmlNodeType.EndElement:
							if (xml.Name == "testrun")
							{
								return; //quit to avoid junk at the end of the output
							}
							break;
						default:
							break;
					}
				}
			}
		}

		void ParseTestCaseResult(XmlReader xml, TestResult testResult)
		{
			xml.Read();
			while (xml.NodeType != XmlNodeType.EndElement)
			{
				if (xml.Name == "failure")
				{
					string failure = xml.ReadElementContentAsString()?.Trim() ?? String.Empty;
					testResult.ErrorStackTrace = failure;
					testResult.Outcome = TestOutcome.Failed;
				}
				else if (xml.Name == "result")
				{
					string strSuccess = xml.GetAttribute("success");
					bool.TryParse(strSuccess, out bool success);
					testResult.Outcome = success ? TestOutcome.Passed : TestOutcome.Failed;

					string strDuration = xml.GetAttribute("duration");
					if (strDuration != null)
					{
						double.TryParse(strDuration, out double duraction);
						testResult.Duration = TimeSpan.FromSeconds(duraction);
					}

					xml.Read();
				}
				else
				{
					xml.Skip();
				}
			}
		}
	}
}
