// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Reflection;
using System.Resources;
using System.Runtime.InteropServices;

// General Information about an assembly is controlled through the following 
// set of attributes. Change these attribute values to modify the information
// associated with an assembly.
[assembly: AssemblyTitle("UnrealVS")]
[assembly: AssemblyDescription("")]
[assembly: AssemblyConfiguration("")]

[assembly: AssemblyCompany("Epic Games, Inc.")]
[assembly: AssemblyProduct("UnrealVS")]
[assembly: AssemblyCopyright("Copyright Epic Games, Inc.")]
[assembly: AssemblyTrademark("")]
[assembly: AssemblyCulture("")]
[assembly: ComVisible(false)]
[assembly: CLSCompliant(false)]
[assembly: NeutralResourcesLanguage("en-US")]

// Version information for an assembly consists of the following four values:
//
//      Major Version
//      Minor Version 
//      Build Number
//      Revision
//
// You can specify all the values or you can default the Revision and Build Numbers 
// by using the '*' as shown below:

[assembly: AssemblyVersion("1.87.0.0")]
[assembly: AssemblyFileVersion("1.87.0.0")]

