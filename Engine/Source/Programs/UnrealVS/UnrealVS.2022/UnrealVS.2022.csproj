<?xml version="1.0" encoding="utf-8"?>
<Project DefaultTargets="Build" xmlns="http://schemas.microsoft.com/developer/msbuild/2003" ToolsVersion="14.0">
  <Import Project="$(MSBuildExtensionsPath)\$(MSBuildToolsVersion)\Microsoft.Common.props" Condition="Exists('$(MSBuildExtensionsPath)\$(MSBuildToolsVersion)\Microsoft.Common.props')" />
  <PropertyGroup>
    <Configuration Condition=" '$(Configuration)' == '' ">Debug</Configuration>
    <Platform Condition=" '$(Platform)' == '' ">AnyCPU</Platform>
    <SchemaVersion>2.0</SchemaVersion>
    <ProjectGuid>{706654CD-5918-429F-970A-011916302127}</ProjectGuid>
    <ProjectTypeGuids>{82b43b9b-a64c-4715-b499-d71e9ca2bd60};{60dc8134-eba5-43b8-bcc9-bb4bc16c2548};{FAE04EC0-301F-11D3-BF4B-00C04F79EFBC}</ProjectTypeGuids>
    <OutputType>Library</OutputType>
    <AppDesignerFolder>Properties</AppDesignerFolder>
    <RootNamespace>UnrealVS</RootNamespace>
    <AssemblyName>UnrealVS</AssemblyName>
    <SignAssembly>True</SignAssembly>
    <AssemblyOriginatorKeyFile>Key.snk</AssemblyOriginatorKeyFile>
    <TargetFrameworkVersion>v4.7.2</TargetFrameworkVersion>
    <SccProjectName>
    </SccProjectName>
    <SccLocalPath>
    </SccLocalPath>
    <SccAuxPath>
    </SccAuxPath>
    <SccProvider>
    </SccProvider>
    <MinimumVisualStudioVersion>15.0</MinimumVisualStudioVersion>
    <OldToolsVersion>14.0</OldToolsVersion>
    <TargetFrameworkProfile />
    <FileUpgradeFlags>
    </FileUpgradeFlags>
    <UpgradeBackupLocation>
    </UpgradeBackupLocation>
    <IsWebBootstrapper>false</IsWebBootstrapper>
    <PublishUrl>publish\</PublishUrl>
    <Install>true</Install>
    <InstallFrom>Disk</InstallFrom>
    <UpdateEnabled>false</UpdateEnabled>
    <UpdateMode>Foreground</UpdateMode>
    <UpdateInterval>7</UpdateInterval>
    <UpdateIntervalUnits>Days</UpdateIntervalUnits>
    <UpdatePeriodically>false</UpdatePeriodically>
    <UpdateRequired>false</UpdateRequired>
    <MapFileExtensions>true</MapFileExtensions>
    <ApplicationRevision>0</ApplicationRevision>
    <ApplicationVersion>1.0.0.%2a</ApplicationVersion>
    <UseApplicationTrust>false</UseApplicationTrust>
    <BootstrapperEnabled>true</BootstrapperEnabled>
  </PropertyGroup>
  <ItemGroup>
    <VSCTCompile Include="UnrealVS.vsct">
      <ResourceName>Menus.ctmenu</ResourceName>
      <SubType>Designer</SubType>
    </VSCTCompile>
  </ItemGroup>
  <ItemGroup>
    <Reference Include="PresentationCore" />
    <Reference Include="PresentationFramework" />
    <Reference Include="System" />
    <Reference Include="System.Core" />
    <Reference Include="System.Data" />
    <Reference Include="System.Design" />
    <Reference Include="System.Drawing" />
    <Reference Include="System.Numerics" />
    <Reference Include="System.Windows.Forms" />
    <Reference Include="System.Xaml" />
    <Reference Include="System.Xml" />
    <Reference Include="WindowsBase" />
  </ItemGroup>
  <ItemGroup>
    <Compile Include="Properties\AssemblyInfo.cs">
    </Compile>
    <Compile Include="Resources.Designer.cs">
      <AutoGen>True</AutoGen>
      <DesignTime>True</DesignTime>
      <DependentUpon>Resources.resx</DependentUpon>
    </Compile>
  </ItemGroup>
  <ItemGroup>
    <EmbeddedResource Include="Resources.resx">
      <Generator>ResXFileCodeGenerator</Generator>
      <LastGenOutput>Resources.Designer.cs</LastGenOutput>
      <SubType>Designer</SubType>
    </EmbeddedResource>
    <EmbeddedResource Include="VSPackage.resx">
      <MergeWithCTO>true</MergeWithCTO>
      <ManifestResourceName>VSPackage</ManifestResourceName>
      <SubType>Designer</SubType>
    </EmbeddedResource>
  </ItemGroup>
  <ItemGroup>
    <None Include="..\..\..\..\Extras\UnrealVS\UnrealVS.vsix">
      <Link>UnrealVS.vsix</Link>
    </None>
    <None Include="Key.snk" />
  </ItemGroup>
  <ItemGroup>
    <Resource Include="Resources\BatchBuilder.png">
    </Resource>
    <Resource Include="Resources\BatchBuilderBackground.png">
    </Resource>
    <Resource Include="Resources\Icon.ico">
    </Resource>
    <Resource Include="..\..\..\..\Extras\UnrealVS\AtomFeed.xml">
      <Link>AtomFeed.xml</Link>
    </Resource>
    <Content Include="Resources\Icon.png">
      <CopyToOutputDirectory>Always</CopyToOutputDirectory>
      <IncludeInVSIX>true</IncludeInVSIX>
    </Content>
    <Resource Include="Resources\Preview.png">
      <IncludeInVSIX>true</IncludeInVSIX>
      <CopyToOutputDirectory>Always</CopyToOutputDirectory>
    </Resource>
    <Resource Include="Resources\RefreshProjects.png">
    </Resource>
    <Resource Include="Resources\FileBrowser.png">
    </Resource>
    <Resource Include="Resources\BookmarkAdd.png">
    </Resource>
    <Resource Include="Resources\BookmarkRemove.png">
    </Resource>
    <Resource Include="Resources\CopyInclude.png">
    </Resource>
    <Resource Include="Resources\InsertInclude.png">
    </Resource>
    <Resource Include="Resources\CycleRelatedFiles.png">
    </Resource>
    <Resource Include="Resources\UnrealIcon.png">
    </Resource>
    <None Include="source.extension.vsixmanifest" />
    <Content Include="..\ReleaseNotes.txt">
      <Visible>false</Visible>
      <CopyToOutputDirectory>Always</CopyToOutputDirectory>
      <IncludeInVSIX>true</IncludeInVSIX>
    </Content>
    <Content Include="..\ReleaseNotes.txt">
      <Link>Resources\ReleaseNotes.txt</Link>
      <CopyToOutputDirectory>Always</CopyToOutputDirectory>
      <IncludeInVSIX>true</IncludeInVSIX>
    </Content>
  </ItemGroup>
  <ItemGroup>
    <BootstrapperPackage Include=".NETFramework,Version=v4.5">
      <Visible>False</Visible>
      <ProductName>Microsoft .NET Framework 4.5 %28x86 and x64%29</ProductName>
      <Install>true</Install>
    </BootstrapperPackage>
    <BootstrapperPackage Include="Microsoft.Net.Client.3.5">
      <Visible>False</Visible>
      <ProductName>.NET Framework 3.5 SP1 Client Profile</ProductName>
      <Install>false</Install>
    </BootstrapperPackage>
    <BootstrapperPackage Include="Microsoft.Net.Framework.3.5.SP1">
      <Visible>False</Visible>
      <ProductName>.NET Framework 3.5 SP1</ProductName>
      <Install>false</Install>
    </BootstrapperPackage>
  </ItemGroup>
  <ItemGroup>
    <PackageReference Include="Microsoft.CodeAnalysis.CSharp">
      <Version>4.10.0</Version>
    </PackageReference>
    <PackageReference Include="Microsoft.CodeAnalysis.NetAnalyzers">
      <Version>8.0.0</Version>
      <IncludeAssets>runtime; build; native; contentfiles; analyzers; buildtransitive</IncludeAssets>
      <PrivateAssets>all</PrivateAssets>
    </PackageReference>
    <PackageReference Include="Microsoft.CSharp">
      <Version>4.7.0</Version>
    </PackageReference>
    <PackageReference Include="Microsoft.VisualStudio.Sdk" Version="17.8.37222">
    </PackageReference>
    <PackageReference Include="Microsoft.VisualStudio.Threading">
      <Version>17.8.14</Version>
    </PackageReference>
    <PackageReference Include="Microsoft.VisualStudio.Threading.Analyzers">
      <Version>17.8.14</Version>
      <IncludeAssets>runtime; build; native; contentfiles; analyzers; buildtransitive</IncludeAssets>
      <PrivateAssets>all</PrivateAssets>
    </PackageReference>
    <PackageReference Include="Microsoft.VisualStudio.VCProjectEngine">
      <Version>17.8.37222</Version>
    </PackageReference>
    <PackageReference Include="Microsoft.VSSDK.BuildTools" Version="17.8.2365">
      <IncludeAssets>runtime; build; native; contentfiles; analyzers; buildtransitive</IncludeAssets>
      <PrivateAssets>all</PrivateAssets>
    </PackageReference>
    <PackageReference Include="System.Text.Json">
      <Version>8.0.5</Version>
    </PackageReference>
  </ItemGroup>
  <ItemGroup>
    <ProjectReference Include="..\Unreal.TestAdapter\Unreal.TestAdapter.csproj">
      <Project>{2571AF3D-AFEE-4A45-B612-2795D5A5FE6B}</Project>
      <Name>Unreal.TestAdapter</Name>
      <IncludeOutputGroupsInVSIX>BuiltProjectOutputGroup%3bBuiltProjectOutputGroupDependencies%3bGetCopyToOutputDirectoryItems%3bSatelliteDllsProjectOutputGroup%3b</IncludeOutputGroupsInVSIX>
    </ProjectReference>
  </ItemGroup>
  <Import Project="..\UnrealVS.Shared\UnrealVS.Shared.projitems" Label="Shared" />
  <PropertyGroup>
    <UseCodebase>true</UseCodebase>
  </PropertyGroup>
  <PropertyGroup>
    <VisualStudioVersion Condition="'$(VisualStudioVersion)' == ''">11.0</VisualStudioVersion>
    <VSToolsPath Condition="'$(VSToolsPath)' == ''">$(MSBuildExtensionsPath32)\Microsoft\VisualStudio\v$(VisualStudioVersion)</VSToolsPath>
  </PropertyGroup>
  <PropertyGroup Condition=" '$(Configuration)|$(Platform)' == 'Debug|AnyCPU' ">
    <DebugSymbols>true</DebugSymbols>
    <OutputPath>bin\Debug\</OutputPath>
    <DefineConstants>TRACE;DEBUG;VS17</DefineConstants>
    <DebugType>full</DebugType>
    <PlatformTarget>AnyCPU</PlatformTarget>
    <ErrorReport>prompt</ErrorReport>
    <CodeAnalysisRuleSet>MinimumRecommendedRules.ruleset</CodeAnalysisRuleSet>
    <NoWarn>$(NoWarn);MSB3277</NoWarn>
    <CopyVsixExtensionFiles>False</CopyVsixExtensionFiles>
  </PropertyGroup>
  <PropertyGroup Condition=" '$(Configuration)|$(Platform)' == 'Release|AnyCPU' ">
    <OutputPath>bin\Release\</OutputPath>
    <DefineConstants>TRACE;VS17</DefineConstants>
    <Optimize>true</Optimize>
    <DebugType>pdbonly</DebugType>
    <PlatformTarget>AnyCPU</PlatformTarget>
    <RunCodeAnalysis>false</RunCodeAnalysis>
    <ErrorReport>prompt</ErrorReport>
    <CodeAnalysisRuleSet>MinimumRecommendedRules.ruleset</CodeAnalysisRuleSet>
    <CopyVsixExtensionFiles>False</CopyVsixExtensionFiles>
    <NoWarn>$(NoWarn);MSB3277</NoWarn>
    <TreatWarningsAsErrors>true</TreatWarningsAsErrors>
    <WarningLevel>5</WarningLevel>
  </PropertyGroup>
  <Target Name="IncludeNuGetPackageReferences" AfterTargets="GetVsixSourceItems">
    <ItemGroup>
      <VSIXSourceItem Include="@(ReferenceCopyLocalPaths)" Condition="'%(ReferenceCopyLocalPaths.NuGetPackageId)' != ''" />
    </ItemGroup>
  </Target>
  <Import Project="$(MSBuildBinPath)\Microsoft.CSharp.targets" />
  <Import Project="$(VSToolsPath)\VSSDK\Microsoft.VsSDK.targets" Condition="'$(VSToolsPath)' != ''" />
  <Import Project="$(MSBuildExtensionsPath32)\Microsoft\VisualStudio\v11.0\VSSDK\Microsoft.VsSDK.targets" Condition="false" />
  <PropertyGroup>
    <PostBuildEvent>if "$(ConfigurationName)" == "Release" xcopy /F /Y "$(TargetDir)$(TargetName).vsix" "$(ProjectDir)..\..\..\..\Extras\UnrealVS\$(TargetName).vsix"</PostBuildEvent>
  </PropertyGroup>
  <PropertyGroup>
    <PreBuildEvent>
    </PreBuildEvent>
  </PropertyGroup>
  <!-- To modify your build process, add your task inside one of the targets below and uncomment it. 
       Other similar extension points exist, see Microsoft.Common.targets.
  <Target Name="BeforeBuild">
  </Target>
  <Target Name="AfterBuild">
  </Target>
  -->
</Project>