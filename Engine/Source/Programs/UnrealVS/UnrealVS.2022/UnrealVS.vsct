<?xml version="1.0" encoding="utf-8"?>
<CommandTable xmlns="http://schemas.microsoft.com/VisualStudio/2005-10-18/CommandTable" xmlns:xs="http://www.w3.org/2001/XMLSchema">

	<!-- This header contains the command IDs for the menus provided by Visual Studio (like guidSHLMainMenu.) -->
	<Extern href="vsshlids.h" />
	
	<!-- VS Icons -->
	<Include href="KnownImageIds.vsct"/>

	<!-- This is the file that defines the IDs for all the commands exposed by VisualStudio. -->
	<!--<Extern href="stdidcmd.h"/> -->

	<!-- Definition of some VSCT specific constants. In this sample we use it for the IDs inside the guidOfficeIcon group. -->
	<!--<Extern href="msobtnid.h"/> -->


	<!-- The Commands section is where we the commands, menus and menu groups are defined. This section 
		 uses a Guid to identify the package that provides the command defined inside it. 
		 Inside this section we have different sub-sections: one for the menus, another  
		 for the menu groups, one for the buttons (the actual commands), one for the combos 
		 and the last one for the bitmaps used. Each element is identified by a command id that  
		 is a unique pair of guid and numeric identifier; the guid part of the identifier is usually  
		 called "command set" and is used to group different command inside a logically related  
		 group; your package should define its own command set in order to avoid collisions  
		 with command ids defined by other packages. -->
	<Commands package="UnrealVSPackageGUID">


		<!-- Buttons section.  This section defines the elements the user can interact with, like a menu 
			 command or a button or combo box in a toolbar. -->
		<Buttons>
			<!-- Project UnrealVS.UBT -->
			<Button guid="UnrealVSCmdSetGUID" id="ProjectRebuildAndProfileButtonID" priority="0x0102" type="Button">
				<Parent guid="UnrealVSCmdSetGUID" id="ProjectUnrealVSUBTGroupID" />
				<Icon guid="ImageCatalogGuid" id="BuildSolution" />
				<CommandFlag>IconIsMoniker</CommandFlag>
				<Strings>
					<ButtonText>Rebuild and Profile</ButtonText>
					<CommandName>Rebuild and Profile</CommandName>
					<ToolTipText>Requires the CompileScore plugin to be installed</ToolTipText>
				</Strings>
			</Button>

			<!-- Folder UnrealVS.UBT -->
			<Button guid="UnrealVSCmdSetGUID" id="FolderUnrealVSBuildButtonID" priority="0x0100" type="Button">
				<Parent guid="UnrealVSCmdSetGUID" id="FolderUnrealVSUBTGroupID" />
				<Icon guid="ImageCatalogGuid" id="BuildSolution" />
				<CommandFlag>IconIsMoniker</CommandFlag>
				<Strings>
					<ButtonText>Build Module</ButtonText>
					<CommandName>Build Module</CommandName>
				</Strings>
			</Button>

			<Button guid="UnrealVSCmdSetGUID" id="FolderUnrealVSBuildNonUnityButtonID" priority="0x0101" type="Button">
				<Parent guid="UnrealVSCmdSetGUID" id="FolderUnrealVSUBTGroupID" />
				<Icon guid="ImageCatalogGuid" id="BuildSolution" />
				<CommandFlag>IconIsMoniker</CommandFlag>
				<Strings>
					<ButtonText>Build Module NonUnity</ButtonText>
					<CommandName>Build Module NonUnity</CommandName>
				</Strings>
			</Button>
			
			<Button guid="UnrealVSCmdSetGUID" id="FolderUnrealVSRebuildAndProfileButtonID" priority="0x0102" type="Button">
				<Parent guid="UnrealVSCmdSetGUID" id="FolderUnrealVSUBTGroupID" />
				<Icon guid="ImageCatalogGuid" id="BuildSolution" />
				<CommandFlag>IconIsMoniker</CommandFlag>
				<Strings>
					<ButtonText>Rebuild and Profile Module</ButtonText>
					<CommandName>Rebuild and Profile Module</CommandName>
				</Strings>
			</Button>

			<Button guid="UnrealVSCmdSetGUID" id="FolderUnrealVSRebuildAndProfileNonUnityButtonID" priority="0x0103" type="Button">
				<Parent guid="UnrealVSCmdSetGUID" id="FolderUnrealVSUBTGroupID" />
				<Icon guid="ImageCatalogGuid" id="BuildSolution" />
				<CommandFlag>IconIsMoniker</CommandFlag>
				<Strings>
					<ButtonText>Rebuild and Profile Module NonUnity</ButtonText>
					<CommandName>Rebuild and Profile Module NonUnity</CommandName>
				</Strings>
			</Button>

			<!-- UBTSubMenuGroup items -->

			<!-- CompileSingleFile -->
			<Button guid="UnrealVSCmdSetGUID" id="CompileSingleFileButtonID" priority="0x100" type="Button">
				<Parent guid="UnrealVSCmdSetGUID" id="UBTSubMenuGroup" />
				<Icon guid="ImageCatalogGuid" id="Compile" />
				<CommandFlag>IconIsMoniker</CommandFlag>
				<Strings>
					<ButtonText>Compile Single File</ButtonText>
					<CommandName>Compile Single File</CommandName>
					<ToolTipText>Only Valid for Source Files (*.c;*.cc;*.cpp;*.h;*.cxx;*.ispc)</ToolTipText>
				</Strings>
			</Button>

			<!--// CompileAndProfileSingleFile -->
			<Button guid="UnrealVSCmdSetGUID" id="CompileAndProfileSingleFileButtonID" priority="0x101" type="Button">
				<Parent guid="UnrealVSCmdSetGUID" id="UBTSubMenuGroup" />
				<Icon guid="ImageCatalogGuid" id="Compile" />
				<CommandFlag>IconIsMoniker</CommandFlag>
				<Strings>
					<ButtonText>Compile And Profile Single File</ButtonText>
					<CommandName>Compile And Profile Single File</CommandName>
					<ToolTipText>Only Valid for Source Files (*.c;*.cc;*.cpp;*.cxx)\r\nRequires the CompileScore plugin to be installed</ToolTipText>
				</Strings>
			</Button>

			<!-- PreprocessSingleFile -->
			<Button guid="UnrealVSCmdSetGUID" id="GenerateAssemblyFileButtonID" priority="0x103" type="Button">
				<Parent guid="UnrealVSCmdSetGUID" id="UBTSubMenuGroup" />
				<Icon guid="ImageCatalogGuid" id="Compile" />
				<CommandFlag>IconIsMoniker</CommandFlag>
				<Strings>
					<ButtonText>Generate Assembly</ButtonText>
					<CommandName>Generate Assembly</CommandName>
					<ToolTipText>Only Valid for Source Files (*.c;*.cc;*.cpp;*.cxx)</ToolTipText>
				</Strings>
			</Button>

			<!-- PreprocessSingleFile -->
			<Button guid="UnrealVSCmdSetGUID" id="PreprocessSingleFileButtonID" priority="0x105" type="Button">
				<Parent guid="UnrealVSCmdSetGUID" id="UBTSubMenuGroup" />
				<Icon guid="ImageCatalogGuid" id="CompilableFile" />
				<CommandFlag>IconIsMoniker</CommandFlag>
				<Strings>
					<ButtonText>Preprocess Single File</ButtonText>
					<CommandName>Preprocess Single File</CommandName>
				</Strings>
			</Button>

			<!-- CompileSingleModule -->
			<Button guid="UnrealVSCmdSetGUID" id="CompileSingleModuleButtonID" priority="0x110" type="Button">
				<Parent guid="UnrealVSCmdSetGUID" id="UBTSubMenuGroup" />
				<Icon guid="ImageCatalogGuid" id="BuildDefinition" />
				<CommandFlag>IconIsMoniker</CommandFlag>
				<Strings>
					<ButtonText>Compile Current Module</ButtonText>
					<CommandName>Compile Current Module</CommandName>
					<ToolTipText>Only Compiles the Module for Current File</ToolTipText>
				</Strings>
			</Button>
			
			<!-- BuildStartupProject -->
			<Button guid="UnrealVSCmdSetGUID" id="BuildStartupProjectButtonID" priority="0x120" type="Button">
				<Parent guid="UnrealVSCmdSetGUID" id="UBTSubMenuGroup" />
				<Icon guid="ImageCatalogGuid" id="UserBuild" />
				<CommandFlag>IconIsMoniker</CommandFlag>
				<Strings>
					<ButtonText>Build Startup Project</ButtonText>
					<CommandName>Build Startup Project</CommandName>
				</Strings>
			</Button>

			<!-- P4SubMenuGroup  items -->
			<!-- P4Checkout -->
			<Button guid="UnrealVSCmdSetGUID" id="P4CheckoutButtonID" priority="0x0100" type="Button">
				<Parent guid="UnrealVSCmdSetGUID" id="P4SubMenuGroup" />
				<Icon guid="ImageCatalogGuid" id="Checkmark" />
				<CommandFlag>DynamicVisibility</CommandFlag>
				<CommandFlag>DefaultInvisible</CommandFlag>
				<CommandFlag>DontCache</CommandFlag>
				<CommandFlag>IconIsMoniker</CommandFlag>
				<Strings>
					<ButtonText>Checkout</ButtonText>
					<CommandName>P4Checkout</CommandName>
				</Strings>
			</Button>

			<!-- P4Revert -->
			<Button guid="UnrealVSCmdSetGUID" id="P4RevertButtonID" priority="0x0100" type="Button">
				<Parent guid="UnrealVSCmdSetGUID" id="P4SubMenuGroup" />
				<Icon guid="ImageCatalogGuid" id="Undo" />
				<CommandFlag>DynamicVisibility</CommandFlag>
				<CommandFlag>DefaultInvisible</CommandFlag>
				<CommandFlag>DontCache</CommandFlag>
				<CommandFlag>IconIsMoniker</CommandFlag>
				<Strings>
					<ButtonText>Revert</ButtonText>
					<CommandName>P4Revert</CommandName>
				</Strings>
			</Button>

			<!-- P4Annotate -->
			<Button guid="UnrealVSCmdSetGUID" id="P4AnnotateButtonID" priority="0x0101" type="Button">
				<Parent guid="UnrealVSCmdSetGUID" id="P4SubMenuGroup" />
				<Icon guid="ImageCatalogGuid" id="SketchflowAnnotation" />
				<CommandFlag>DynamicVisibility</CommandFlag>
				<CommandFlag>DefaultInvisible</CommandFlag>
				<CommandFlag>DontCache</CommandFlag>
				<CommandFlag>IconIsMoniker</CommandFlag>
				<Strings>
					<ButtonText>Annotate</ButtonText>
					<CommandName>P4Annotate</CommandName>
				</Strings>
			</Button>

			<!-- P4ViewSelectedCL -->
			<Button guid="UnrealVSCmdSetGUID" id="P4ViewSelectedCLButtonID" priority="0x0102" type="Button">
				<Parent guid="UnrealVSCmdSetGUID" id="P4SubMenuGroup" />
				<Icon guid="ImageCatalogGuid" id="DescriptionViewer" />
				<CommandFlag>DynamicVisibility</CommandFlag>
				<CommandFlag>DefaultInvisible</CommandFlag>
				<CommandFlag>DontCache</CommandFlag>
				<CommandFlag>IconIsMoniker</CommandFlag>
				<Strings>
					<ButtonText>View Selected CL</ButtonText>
					<CommandName>P4ViewSelectedCL</CommandName>
					<ToolTipText>Show changelist description for selected text number</ToolTipText>
				</Strings>
			</Button>

			<!-- P4IntegrationAwareTimelapse -->
			<Button guid="UnrealVSCmdSetGUID" id="P4IntegrationAwareTimelapseButtonID" priority="0x0102" type="Button">
				<Parent guid="UnrealVSCmdSetGUID" id="P4SubMenuGroup" />
				<CommandFlag>DynamicVisibility</CommandFlag>
				<CommandFlag>DefaultInvisible</CommandFlag>
				<CommandFlag>DontCache</CommandFlag>
				<Strings>
					<ButtonText>P4 Integration Aware Time-lapse</ButtonText>
					<CommandName>P4IntegrationAwareTimelapse</CommandName>
				</Strings>
			</Button>

			<!-- P4Timelapse -->
			<Button guid="UnrealVSCmdSetGUID" id="P4TimelapseButtonID" priority="0x0102" type="Button">
				<Parent guid="UnrealVSCmdSetGUID" id="P4SubMenuGroup" />
				<Icon guid="ImageCatalogGuid" id="Time" />
				<CommandFlag>DynamicVisibility</CommandFlag>
				<CommandFlag>DefaultInvisible</CommandFlag>
				<CommandFlag>DontCache</CommandFlag>
				<CommandFlag>IconIsMoniker</CommandFlag>
				<Strings>
					<ButtonText>Time-lapse View</ButtonText>
					<CommandName>P4Timelapse</CommandName>
				</Strings>
			</Button>

			<!-- P4RevisionGraph -->
			<Button guid="UnrealVSCmdSetGUID" id="P4RevisionGraphButtonID" priority="0x0102" type="Button">
				<Parent guid="UnrealVSCmdSetGUID" id="P4SubMenuGroup" />
				<Icon guid="ImageCatalogGuid" id="Timeline" />
				<CommandFlag>DynamicVisibility</CommandFlag>
				<CommandFlag>DefaultInvisible</CommandFlag>
				<CommandFlag>DontCache</CommandFlag>
				<CommandFlag>IconIsMoniker</CommandFlag>
				<Strings>
					<ButtonText>Revision Graph</ButtonText>
					<CommandName>P4RevisionGraph</CommandName>
				</Strings>
			</Button>

			<!-- P4FileHistory -->
			<Button guid="UnrealVSCmdSetGUID" id="P4FileHistoryButtonID" priority="0x0102" type="Button">
				<Parent guid="UnrealVSCmdSetGUID" id="P4SubMenuGroup" />
				<Icon guid="ImageCatalogGuid" id="History" />
				<CommandFlag>DynamicVisibility</CommandFlag>
				<CommandFlag>DefaultInvisible</CommandFlag>
				<CommandFlag>DontCache</CommandFlag>
				<CommandFlag>IconIsMoniker</CommandFlag>
				<Strings>
					<ButtonText>File History</ButtonText>
					<CommandName>P4FileHistory</CommandName>
				</Strings>
			</Button>

			<!-- P4IDiffinVS -->
			<Button guid="UnrealVSCmdSetGUID" id="P4DiffButtonID" priority="0x0103" type="Button">
				<Parent guid="UnrealVSCmdSetGUID" id="P4SubMenuGroup" />
				<Icon guid="ImageCatalogGuid" id="CompareFiles" />
				<CommandFlag>DynamicVisibility</CommandFlag>
				<CommandFlag>DefaultInvisible</CommandFlag>
				<CommandFlag>DontCache</CommandFlag>
				<CommandFlag>IconIsMoniker</CommandFlag>
				<Strings>
					<ButtonText>Diff Against Have Revision</ButtonText>
					<CommandName>P4Diff</CommandName>
				</Strings>
			</Button>


			<!-- P4GetLast10Changes -->
			<Button guid="UnrealVSCmdSetGUID" id="P4GetLast10ChangesID" priority="0x0103" type="Button">
				<Parent guid="UnrealVSCmdSetGUID" id="P4SubMenuGroup" />
				<Icon guid="ImageCatalogGuid" id="LastHistoryCommand" />
				<CommandFlag>DynamicVisibility</CommandFlag>
				<CommandFlag>DefaultInvisible</CommandFlag>
				<CommandFlag>DontCache</CommandFlag>
				<CommandFlag>IconIsMoniker</CommandFlag>
				<Strings>
					<ButtonText>Get Last 10 Changes</ButtonText>
					<CommandName>P4GetLast10Changes</CommandName>
				</Strings>
			</Button>

			<!-- P4ShowFileinP4V -->
			<Button guid="UnrealVSCmdSetGUID" id="P4ShowFileinP4VID" priority="0x0103" type="Button">
				<Parent guid="UnrealVSCmdSetGUID" id="P4SubMenuGroup" />
				<Icon guid="ImageCatalogGuid" id="OpenFileDialog" />
				<CommandFlag>DynamicVisibility</CommandFlag>
				<CommandFlag>DefaultInvisible</CommandFlag>
				<CommandFlag>DontCache</CommandFlag>
				<CommandFlag>IconIsMoniker</CommandFlag>
				<Strings>
					<ButtonText>Show File in P4V</ButtonText>
					<CommandName>P4ShowFileinP4V</CommandName>
				</Strings>
			</Button>

			<!-- P4FastReconcileCodeFiles -->
			<Button guid="UnrealVSCmdSetGUID" id="P4FastReconcileCodeFiles" priority="0x0104" type="Button">
				<Parent guid="UnrealVSCmdSetGUID" id="P4SubMenuGroup" />
				<CommandFlag>DynamicVisibility</CommandFlag>
				<CommandFlag>DefaultInvisible</CommandFlag>
				<CommandFlag>DontCache</CommandFlag>
				<Strings>
					<ButtonText>Fast Reconcile Code Files</ButtonText>
					<CommandName>P4FastReconcileCodeFiles</CommandName>
				</Strings>
			</Button>

			<!-- GenerateProjectFiles - normal button (available in UnrealVS tool bar, but not keybindable) -->
			<Button guid="UnrealVSCmdSetGUID" id="GenerateProjectFilesButtonID" priority="100" type="Button">

				<!-- Parent the button to our tool bar -->
				<Parent guid="UnrealVSCmdSetGUID" id="CommandsMenuGroupID" />
				<Icon guid="RefreshProjectsIconGUID" id="RefreshProjectsIcon" />
				<CommandFlag>CommandWellOnly</CommandFlag>
				<CommandFlag>Pict</CommandFlag>

				<Strings>
					<ButtonText>Refresh Projects</ButtonText>
					<CommandName>RefreshProjectFiles</CommandName>
				</Strings>
			</Button>

			<!-- BatchBuilder - normal button (available in UnrealVS tool bar, but not keybindable) -->
			<Button guid="UnrealVSCmdSetGUID" id="BatchBuilderToolWindowId" priority="101" type="Button">
				<!-- Parent the button to our tool bar -->
				<Parent guid="UnrealVSCmdSetGUID" id="CommandsMenuGroupID" />
				<Icon guid="BatchBuilderIconGUID" id="BatchBuilderIcon" />
				<CommandFlag>CommandWellOnly</CommandFlag>
				<CommandFlag>Pict</CommandFlag>
				<Strings>
					<ButtonText>Batch Builder</ButtonText>
					<CommandName>OpenBatchBuilderWindow</CommandName>
				</Strings>
			</Button>

			<!-- FileBrowser -->
			<Button guid="UnrealVSCmdSetGUID" id="FileBrowserButtonID" priority="0" type="Button">
				<Parent guid="UnrealVSCmdSetGUID" id="CommandsMenuGroupID" />
				<Icon guid="FileBrowserIconGUID" id="FileBrowserIcon" />
				<Strings>
					<ButtonText>File Browser</ButtonText>
					<CommandName>FileBrowser</CommandName>
				</Strings>
			</Button>

			<!-- CycleRelatedFiles -->
			<Button guid="UnrealVSCmdSetGUID" id="CycleRelatedFilesButtonID" priority="0" type="Button">
				<Parent guid="UnrealVSCmdSetGUID" id="CommandsMenuGroupID" />
				<Icon guid="CycleRelatedFilesIconGUID" id="CycleRelatedFilesIcon" />
				<Strings>
					<ButtonText>Cycle Related Files</ButtonText>
					<CommandName>CycleRelatedFiles</CommandName>
				</Strings>
			</Button>

			<!-- UbaVisualizer -->
			<Button guid="UnrealVSCmdSetGUID" id="UbaVisualizerButtonID" priority="0" type="Button">
				<Parent guid="UnrealVSCmdSetGUID" id="CommandsMenuGroupID" />
				<Icon guid="ImageCatalogGuid" id="ShowAllFiles" />
				<CommandFlag>IconIsMoniker</CommandFlag>
				<Strings>
					<ButtonText>UbaVisualizer</ButtonText>
					<CommandName>UbaVisualizer</CommandName>
				</Strings>
			</Button>

			<!-- CopyIncludePath - document tab context menu -->
			<Button guid="UnrealVSCmdSetGUID" id="CopyIncludePathButtonID" priority="0x0100" type="Button">
				<Parent guid="UnrealVSCmdSetGUID" id="CopyIncludePathGroupID" />
				<Icon guid="UnrealIconGUID" id="UnrealIcon" />
				<CommandFlag>DynamicVisibility</CommandFlag>
				<CommandFlag>DefaultInvisible</CommandFlag>
				<Strings>
					<ButtonText>Copy Unreal #include Statement</ButtonText>
					<CommandName>CopyIncludePath</CommandName>
					<ToolTipText>Copy a #include directive for this header to the clipboard</ToolTipText>
				</Strings>
			</Button>

			<!-- Quick build dynamic placeholder button -->
			<!-- These buttons are populated with the platforms currently available dynamically-->
			<!-- Debug -->
			<Button guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildDebugDynamicStartCommandID" priority="0x0000">
				<Parent guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildDebugSubMenuGroupID" />
				<CommandFlag>DynamicItemStart</CommandFlag>
				<CommandFlag>DynamicVisibility</CommandFlag>
				<CommandFlag>TextChanges</CommandFlag>
				<Strings>
					<!-- This text does not appear. -->
					<ButtonText>DUMMY Win64</ButtonText>
				</Strings>
			</Button>

			<Button guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildDebugClientDynamicStartCommandID" priority="0x0010">
				<Parent guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildDebugClientSubMenuGroupID" />
				<CommandFlag>DynamicItemStart</CommandFlag>
				<CommandFlag>DynamicVisibility</CommandFlag>
				<CommandFlag>TextChanges</CommandFlag>
				<Strings>
					<!-- This text does not appear. -->
					<ButtonText>DUMMY Win64</ButtonText>
				</Strings>
			</Button>

			<Button guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildDebugEditorDynamicStartCommandID" priority="0x0020">
				<Parent guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildDebugEditorSubMenuGroupID" />
				<CommandFlag>DynamicItemStart</CommandFlag>
				<CommandFlag>DynamicVisibility</CommandFlag>
				<CommandFlag>TextChanges</CommandFlag>
				<Strings>
					<!-- This text does not appear. -->
					<ButtonText>DUMMY Win64</ButtonText>
				</Strings>
			</Button>

			<Button guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildDebugServerDynamicStartCommandID" priority="0x0030">
				<Parent guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildDebugServerSubMenuGroupID" />
				<CommandFlag>DynamicItemStart</CommandFlag>
				<CommandFlag>DynamicVisibility</CommandFlag>
				<CommandFlag>TextChanges</CommandFlag>
				<Strings>
					<!-- This text does not appear. -->
					<ButtonText>DUMMY Win64</ButtonText>
				</Strings>
			</Button>

			<!-- DebugGame -->
			<Button guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildDebugGameDynamicStartCommandID" priority="0x0100">
				<Parent guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildDebugGameSubMenuGroupID" />
				<CommandFlag>DynamicItemStart</CommandFlag>
				<CommandFlag>DynamicVisibility</CommandFlag>
				<CommandFlag>TextChanges</CommandFlag>
				<Strings>
					<!-- This text does not appear. -->
					<ButtonText>DUMMY Win64</ButtonText>
				</Strings>
			</Button>

			<Button guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildDebugGameClientDynamicStartCommandID" priority="0x0110">
				<Parent guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildDebugGameClientSubMenuGroupID" />
				<CommandFlag>DynamicItemStart</CommandFlag>
				<CommandFlag>DynamicVisibility</CommandFlag>
				<CommandFlag>TextChanges</CommandFlag>
				<Strings>
					<!-- This text does not appear. -->
					<ButtonText>DUMMY Win64</ButtonText>
				</Strings>
			</Button>

			<Button guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildDebugGameEditorDynamicStartCommandID" priority="0x0120">
				<Parent guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildDebugGameEditorSubMenuGroupID" />
				<CommandFlag>DynamicItemStart</CommandFlag>
				<CommandFlag>DynamicVisibility</CommandFlag>
				<CommandFlag>TextChanges</CommandFlag>
				<Strings>
					<!-- This text does not appear. -->
					<ButtonText>DUMMY Win64</ButtonText>
				</Strings>
			</Button>

			<Button guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildDebugGameServerDynamicStartCommandID" priority="0x0130">
				<Parent guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildDebugGameServerSubMenuGroupID" />
				<CommandFlag>DynamicItemStart</CommandFlag>
				<CommandFlag>DynamicVisibility</CommandFlag>
				<CommandFlag>TextChanges</CommandFlag>
				<Strings>
					<!-- This text does not appear. -->
					<ButtonText>DUMMY Win64</ButtonText>
				</Strings>
			</Button>

			<!-- Development -->
			<Button guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildDevelopmentDynamicStartCommandID" priority="0x0200">
				<Parent guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildDevelopmentSubMenuGroupID" />
				<CommandFlag>DynamicItemStart</CommandFlag>
				<CommandFlag>DynamicVisibility</CommandFlag>
				<CommandFlag>TextChanges</CommandFlag>
				<Strings>
					<!-- This text does not appear. -->
					<ButtonText>DUMMY Win64</ButtonText>
				</Strings>
			</Button>

			<Button guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildDevelopmentClientDynamicStartCommandID" priority="0x0210">
				<Parent guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildDevelopmentClientSubMenuGroupID" />
				<CommandFlag>DynamicItemStart</CommandFlag>
				<CommandFlag>DynamicVisibility</CommandFlag>
				<CommandFlag>TextChanges</CommandFlag>
				<Strings>
					<!-- This text does not appear. -->
					<ButtonText>DUMMY Win64</ButtonText>
				</Strings>
			</Button>

			<Button guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildDevelopmentEditorDynamicStartCommandID" priority="0x0220">
				<Parent guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildDevelopmentEditorSubMenuGroupID" />
				<CommandFlag>DynamicItemStart</CommandFlag>
				<CommandFlag>DynamicVisibility</CommandFlag>
				<CommandFlag>TextChanges</CommandFlag>
				<Strings>
					<!-- This text does not appear. -->
					<ButtonText>DUMMY Win64</ButtonText>
				</Strings>
			</Button>

			<Button guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildDevelopmentServerDynamicStartCommandID" priority="0x0230">
				<Parent guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildDevelopmentServerSubMenuGroupID" />
				<CommandFlag>DynamicItemStart</CommandFlag>
				<CommandFlag>DynamicVisibility</CommandFlag>
				<CommandFlag>TextChanges</CommandFlag>
				<Strings>
					<!-- This text does not appear. -->
					<ButtonText>DUMMY Win64</ButtonText>
				</Strings>
			</Button>

			<!-- Shipping -->
			<Button guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildShippingDynamicStartCommandID" priority="0x0300">
				<Parent guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildShippingSubMenuGroupID" />
				<CommandFlag>DynamicItemStart</CommandFlag>
				<CommandFlag>DynamicVisibility</CommandFlag>
				<CommandFlag>TextChanges</CommandFlag>
				<Strings>
					<!-- This text does not appear. -->
					<ButtonText>DUMMY Win64</ButtonText>
				</Strings>
			</Button>

			<Button guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildShippingClientDynamicStartCommandID" priority="0x0310">
				<Parent guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildShippingClientSubMenuGroupID" />
				<CommandFlag>DynamicItemStart</CommandFlag>
				<CommandFlag>DynamicVisibility</CommandFlag>
				<CommandFlag>TextChanges</CommandFlag>
				<Strings>
					<!-- This text does not appear. -->
					<ButtonText>DUMMY Win64</ButtonText>
				</Strings>
			</Button>

			<Button guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildShippingEditorDynamicStartCommandID" priority="0x0320">
				<Parent guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildShippingEditorSubMenuGroupID" />
				<CommandFlag>DynamicItemStart</CommandFlag>
				<CommandFlag>DynamicVisibility</CommandFlag>
				<CommandFlag>TextChanges</CommandFlag>
				<Strings>
					<!-- This text does not appear. -->
					<ButtonText>DUMMY Win64</ButtonText>
				</Strings>
			</Button>

			<Button guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildShippingServerDynamicStartCommandID" priority="0x0330">
				<Parent guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildShippingServerSubMenuGroupID" />
				<CommandFlag>DynamicItemStart</CommandFlag>
				<CommandFlag>DynamicVisibility</CommandFlag>
				<CommandFlag>TextChanges</CommandFlag>
				<Strings>
					<!-- This text does not appear. -->
					<ButtonText>DUMMY Win64</ButtonText>
				</Strings>
			</Button>

			<!-- Test -->
			<Button guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildTestDynamicStartCommandID" priority="0x0300">
				<Parent guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildTestSubMenuGroupID" />
				<CommandFlag>DynamicItemStart</CommandFlag>
				<CommandFlag>DynamicVisibility</CommandFlag>
				<CommandFlag>TextChanges</CommandFlag>
				<Strings>
					<!-- This text does not appear. -->
					<ButtonText>DUMMY Win64</ButtonText>
				</Strings>
			</Button>

			<Button guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildTestClientDynamicStartCommandID" priority="0x0310">
				<Parent guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildTestClientSubMenuGroupID" />
				<CommandFlag>DynamicItemStart</CommandFlag>
				<CommandFlag>DynamicVisibility</CommandFlag>
				<CommandFlag>TextChanges</CommandFlag>
				<Strings>
					<!-- This text does not appear. -->
					<ButtonText>DUMMY Win64</ButtonText>
				</Strings>
			</Button>

			<Button guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildTestEditorDynamicStartCommandID" priority="0x0320">
				<Parent guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildTestEditorSubMenuGroupID" />
				<CommandFlag>DynamicItemStart</CommandFlag>
				<CommandFlag>DynamicVisibility</CommandFlag>
				<CommandFlag>TextChanges</CommandFlag>
				<Strings>
					<!-- This text does not appear. -->
					<ButtonText>DUMMY Win64</ButtonText>
				</Strings>
			</Button>

			<Button guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildTestServerDynamicStartCommandID" priority="0x0330">
				<Parent guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildTestServerSubMenuGroupID" />
				<CommandFlag>DynamicItemStart</CommandFlag>
				<CommandFlag>DynamicVisibility</CommandFlag>
				<CommandFlag>TextChanges</CommandFlag>
				<Strings>
					<!-- This text does not appear. -->
					<ButtonText>DUMMY Win64</ButtonText>
				</Strings>
			</Button>
		</Buttons>


		<!-- Combos are defined in the Combos section. Each Combo element represents a drop-down list box
			 in the IDE. The list box may or may not be writable by users, depending on the value of the 
			 type attribute of the combo. Combos have the same elements and behavior that buttons have, with 
			 some additional features -->
		<Combos>

			<!-- Combo box for selecting a new startup project -->
			<!-- Command list version of this combo - adds the command to the commands list for customization -->
			<Combo guid="UnrealVSCmdSetGUID" id="StartupProjectComboID" priority="10" type="DynamicCombo" idCommandList="StartupProjectComboListID" defaultWidth="160">

				<!-- Parent the button to the tool bar -->
				<Parent guid="UnrealVSCmdSetGUID" id="CommandsMenuGroupID" />

				<!-- Allow user to customize even though it's not in the menu -->
				<CommandFlag>CommandWellOnly</CommandFlag>

				<Strings>
					<ButtonText>Startup Project</ButtonText>
					<CommandName>Startup Project Selector</CommandName>
				</Strings>
			</Combo>

			<!-- Combo box for selecting a new startup project -->
			<!-- UnrealVS Toolbar version of this combo -->
			<Combo guid="UnrealVSCmdSetGUID" id="StartupProjectComboID" priority="10" type="DynamicCombo" idCommandList="StartupProjectComboListID" defaultWidth="160">

				<!-- Parent the button to the tool bar -->
				<Parent guid="UnrealVSCmdSetGUID" id="UnrealVSToolbarGroupID" />

				<!-- Allow user to customize even though it's not in the menu -->
				<CommandFlag>CommandWellOnly</CommandFlag>
				<CommandFlag>DefaultDisabled</CommandFlag>

				<Strings>
					<ButtonText>Startup Project</ButtonText>
					<CommandName>Startup Project Selector</CommandName>
				</Strings>
			</Combo>

			<!-- Combo box for selecting command-line arguments and typing in new command-line arguments -->
			<!-- Command list version of this combo - adds the command to the commands list for customization -->
			<Combo guid="UnrealVSCmdSetGUID" id="CommandLineComboID" priority="20" type="DynamicCombo" idCommandList="CommandLineComboListID" defaultWidth="200">

				<!-- Parent the button to the tool bar -->
				<Parent guid="UnrealVSCmdSetGUID" id="CommandsMenuGroupID" />

				<!-- Allow user to customize even though it's not in the menu -->
				<CommandFlag>CommandWellOnly</CommandFlag>

				<!-- Callback to the handler for every keystroke -->
				<CommandFlag>FilterKeys</CommandFlag>

				<Strings>
					<ButtonText>Command-line</ButtonText>
					<CommandName>Command Line Editor</CommandName>
				</Strings>
			</Combo>

			<!-- Combo box for selecting command-line arguments and typing in new command-line arguments -->
			<!-- UnrealVS Toolbar version of this combo -->
			<Combo guid="UnrealVSCmdSetGUID" id="CommandLineComboID" priority="20" type="DynamicCombo" idCommandList="CommandLineComboListID" defaultWidth="200">

				<!-- Parent the button to the tool bar -->
				<Parent guid="UnrealVSCmdSetGUID" id="UnrealVSToolbarGroupID" />

				<!-- Allow user to customize even though it's not in the menu -->
				<CommandFlag>CommandWellOnly</CommandFlag>
				<CommandFlag>DefaultDisabled</CommandFlag>

				<!-- Callback to the handler for every keystroke -->
				<CommandFlag>FilterKeys</CommandFlag>

				<!-- Stretch to fill remaining space in tool bar (only one combo per tool bar can do this) -->
				<CommandFlag>StretchHorizontally</CommandFlag>

				<Strings>
					<ButtonText>Command-line</ButtonText>
					<CommandName>Command Line Editor</CommandName>
				</Strings>
			</Combo>

		</Combos>


		<!-- Menus section.  In this file, Menus and Toolbars are treated as the same exact thing!
			 To define a menu group you have to specify its ID, the parent menu and its display priority. 
			 The command is visible and enabled by default. If you need to change the visibility, status, 
			 etc, you can use the CommandFlag node. You can add more than one CommandFlag node e.g.:
					<CommandFlag>DefaultInvisible</CommandFlag>
					<CommandFlag>DynamicVisibility</CommandFlag>
			 If you do not want an image next to your command, remove the Icon node or set it to:
					<Icon guid="guidOfficeIcon" id="msotcidNoIcon" /> -->
		<Menus>
			<!-- Fake menu that only contains commands that are never actually displayed in the UI.  We parent commands to this that we want to be visible as customizable commands in the Keyboard Options menu, under the "UnrealVS.xxx" prefix -->
			<Menu guid="UnrealVSCmdSetGUID" id="CommandsMenuID" type="Menu" priority="0x0000">
				<Parent guid="guidSHLMainMenu" id="IDG_VS_MM_TOOLSADDINS" />
				<Strings>
					<ButtonText>UnrealVS</ButtonText>
					<CommandName>UnrealVS</CommandName>
				</Strings>
			</Menu>


			<!-- UnrealVS tool bar -->
			<Menu guid="UnrealVSCmdSetGUID" id="UnrealVSToolbar" type="Toolbar">

				<!--<Parent guid="guidSHLMainMenu" id="IDG_VS_MM_TOOLSADDINS"/>-->
				<CommandFlag>DefaultDocked</CommandFlag>

				<!-- NOTE: We don't need a parent because tool bars are never nested -->

				<Strings>
					<ButtonText>UnrealVS</ButtonText>
					<CommandName>UnrealVS</CommandName>
				</Strings>
			</Menu>

			<Menu guid="UnrealVSCmdSetGUID" id="ProjectUnrealVSUBTMenuID" priority="0x0102" type="Menu">
				<Parent guid="UnrealVSCmdSetGUID" id="UnrealVSProjectGroupID" />
				<Strings>
					<ButtonText>UnrealVS.UBT</ButtonText>
				</Strings>
			</Menu>

			<Menu guid="UnrealVSCmdSetGUID" id="FolderUnrealVSUBTMenuID" priority="0x0102" type="Menu">
				<Parent guid="UnrealVSCmdSetGUID" id="UnrealVSFolderGroupID" />
				<Strings>
					<ButtonText>UnrealVS.UBT</ButtonText>
				</Strings>
			</Menu>

			<!-- UnrealVS Quickbuild submenu -->
			<Menu guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildSubMenuID" priority="0x0101" type="Menu">
				<Parent guid="UnrealVSCmdSetGUID" id="UnrealVSProjectGroupID" />
				<Icon guid="BatchBuilderIconGUID" id="BatchBuilderIcon" />
				<CommandFlag>DynamicVisibility</CommandFlag>
				<CommandFlag>DefaultInvisible</CommandFlag>
				<CommandFlag>DontCache</CommandFlag>
				<Strings>
					<ButtonText>UnrealVS Quick Build</ButtonText>
				</Strings>
			</Menu>

			<!-- UnrealVS Quickbuild configuration submenus -->

			<!-- Debug -->
			<Menu guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildDebugSubMenuID" priority="0x0000" type="Menu">
				<Parent guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildSubMenuGroupID" />
				<CommandFlag>DynamicVisibility</CommandFlag>
				<Strings>
					<ButtonText>Debug</ButtonText>
				</Strings>
			</Menu>

			<Menu guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildDebugClientSubMenuID" priority="0x0010" type="Menu">
				<Parent guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildSubMenuGroupID" />
				<CommandFlag>DynamicVisibility</CommandFlag>
				<Strings>
					<ButtonText>Debug Client</ButtonText>
				</Strings>
			</Menu>

			<Menu guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildDebugEditorSubMenuID" priority="0x0020" type="Menu">
				<Parent guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildSubMenuGroupID" />
				<CommandFlag>DynamicVisibility</CommandFlag>
				<Strings>
					<ButtonText>Debug Editor</ButtonText>
				</Strings>
			</Menu>

			<Menu guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildDebugServerSubMenuID" priority="0x0030" type="Menu">
				<Parent guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildSubMenuGroupID" />
				<CommandFlag>DynamicVisibility</CommandFlag>
				<Strings>
					<ButtonText>Debug Server</ButtonText>
				</Strings>
			</Menu>

			<!-- DebugGame -->
			<Menu guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildDebugGameSubMenuID" priority="0x0100" type="Menu">
				<Parent guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildSubMenuGroupID" />
				<CommandFlag>DynamicVisibility</CommandFlag>
				<Strings>
					<ButtonText>DebugGame</ButtonText>
				</Strings>
			</Menu>

			<Menu guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildDebugGameClientSubMenuID" priority="0x0110" type="Menu">
				<Parent guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildSubMenuGroupID" />
				<CommandFlag>DynamicVisibility</CommandFlag>
				<Strings>
					<ButtonText>DebugGame Client</ButtonText>
				</Strings>
			</Menu>

			<Menu guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildDebugGameEditorSubMenuID" priority="0x0120" type="Menu">
				<Parent guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildSubMenuGroupID" />
				<CommandFlag>DynamicVisibility</CommandFlag>
				<Strings>
					<ButtonText>DebugGame Editor</ButtonText>
				</Strings>
			</Menu>

			<Menu guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildDebugGameServerSubMenuID" priority="0x0130" type="Menu">
				<Parent guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildSubMenuGroupID" />
				<CommandFlag>DynamicVisibility</CommandFlag>
				<Strings>
					<ButtonText>DebugGame Server</ButtonText>
				</Strings>
			</Menu>

			<!-- Development -->
			<Menu guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildDevelopmentSubMenuID" priority="0x0200" type="Menu">
				<Parent guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildSubMenuGroupID" />
				<CommandFlag>DynamicVisibility</CommandFlag>
				<Strings>
					<ButtonText>Development</ButtonText>
				</Strings>
			</Menu>

			<Menu guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildDevelopmentClientSubMenuID" priority="0x0210" type="Menu">
				<Parent guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildSubMenuGroupID" />
				<CommandFlag>DynamicVisibility</CommandFlag>
				<Strings>
					<ButtonText>Development Client</ButtonText>
				</Strings>
			</Menu>

			<Menu guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildDevelopmentEditorSubMenuID" priority="0x0220" type="Menu">
				<Parent guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildSubMenuGroupID" />
				<CommandFlag>DynamicVisibility</CommandFlag>
				<Strings>
					<ButtonText>Development Editor</ButtonText>
				</Strings>
			</Menu>

			<Menu guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildDevelopmentServerSubMenuID" priority="0x0230" type="Menu">
				<Parent guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildSubMenuGroupID" />
				<CommandFlag>DynamicVisibility</CommandFlag>
				<Strings>
					<ButtonText>Development Server</ButtonText>
				</Strings>
			</Menu>

			<!-- Shipping -->
			<Menu guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildShippingSubMenuID" priority="0x0300" type="Menu">
				<Parent guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildSubMenuGroupID" />
				<CommandFlag>DynamicVisibility</CommandFlag>
				<Strings>
					<ButtonText>Shipping</ButtonText>
				</Strings>
			</Menu>

			<Menu guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildShippingClientSubMenuID" priority="0x0310" type="Menu">
				<Parent guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildSubMenuGroupID" />
				<CommandFlag>DynamicVisibility</CommandFlag>
				<Strings>
					<ButtonText>Shipping Client</ButtonText>
				</Strings>
			</Menu>

			<Menu guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildShippingEditorSubMenuID" priority="0x0320" type="Menu">
				<Parent guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildSubMenuGroupID" />
				<CommandFlag>DynamicVisibility</CommandFlag>
				<Strings>
					<ButtonText>Shipping Editor</ButtonText>
				</Strings>
			</Menu>

			<Menu guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildShippingServerSubMenuID" priority="0x0330" type="Menu">
				<Parent guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildSubMenuGroupID" />
				<CommandFlag>DynamicVisibility</CommandFlag>
				<Strings>
					<ButtonText>Shipping Server</ButtonText>
				</Strings>
			</Menu>

			<!-- Test -->
			<Menu guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildTestSubMenuID" priority="0x0400" type="Menu">
				<Parent guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildSubMenuGroupID" />
				<CommandFlag>DynamicVisibility</CommandFlag>
				<Strings>
					<ButtonText>Test</ButtonText>
				</Strings>
			</Menu>

			<Menu guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildTestClientSubMenuID" priority="0x0410" type="Menu">
				<Parent guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildSubMenuGroupID" />
				<CommandFlag>DynamicVisibility</CommandFlag>
				<Strings>
					<ButtonText>Test Client</ButtonText>
				</Strings>
			</Menu>

			<Menu guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildTestEditorSubMenuID" priority="0x0420" type="Menu">
				<Parent guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildSubMenuGroupID" />
				<CommandFlag>DynamicVisibility</CommandFlag>
				<Strings>
					<ButtonText>Test Editor</ButtonText>
				</Strings>
			</Menu>

			<Menu guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildTestServerSubMenuID" priority="0x0430" type="Menu">
				<Parent guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildSubMenuGroupID" />
				<CommandFlag>DynamicVisibility</CommandFlag>
				<Strings>
					<ButtonText>Test Server</ButtonText>
				</Strings>
			</Menu>

			<Menu guid="UnrealVSCmdSetGUID" id="P4SubMenu" priority="0x0" type="Menu">
				<Parent guid="UnrealVSCmdSetGUID" id="CommandsMenuGroupID"/>
				<CommandFlag>DynamicVisibility</CommandFlag>
				<CommandFlag>DefaultInvisible</CommandFlag>
				<CommandFlag>DontCache</CommandFlag>
				<Strings>
					<ButtonText>UnrealVS.P4</ButtonText>
				</Strings>
			</Menu>

			<Menu guid="UnrealVSCmdSetGUID" id="UBTSubMenu" priority="0x01" type="Menu">
				<Parent guid="UnrealVSCmdSetGUID" id="CommandsMenuGroupID"/>
				<CommandFlag>DynamicVisibility</CommandFlag>
				<CommandFlag>DefaultInvisible</CommandFlag>
				<CommandFlag>DontCache</CommandFlag>
				<Strings>
					<ButtonText>UnrealVS.UBT</ButtonText>
				</Strings>
			</Menu>
		</Menus>

		<!-- In this section you can define new menu groups. A menu group is a container for 
			 other menus or buttons (commands); from a visual point of view you can see the 
			 group as the part of a menu contained between two lines. The parent of a group 
			 must be a menu. -->
		<Groups>

			<Group guid="UnrealVSCmdSetGUID" id="CommandsMenuGroupID" priority="0x0000">
				<!-- Parent the menu group to our commands menu -->
				<Parent guid="UnrealVSCmdSetGUID" id="CommandsMenuID" />
			</Group>

			<Group guid="UnrealVSCmdSetGUID" id="UnrealVSToolbarGroupID" priority="0x0000">
				<!-- Parent the tool bar group to the tool bar -->
				<Parent guid="UnrealVSCmdSetGUID" id="UnrealVSToolbar" />
			</Group>

			<Group guid="UnrealVSCmdSetGUID" id="UnrealVSProjectGroupID" priority="0x0000">
				<!-- Parent the menu group to the project node context menu -->
				<Parent guid="guidSHLMainMenu" id="IDM_VS_CTXT_PROJNODE" />
			</Group>

			<Group guid="UnrealVSCmdSetGUID" id="UnrealVSFolderGroupID" priority="0x0000">
				<!-- Parent the menu group to the project node context menu -->
				<Parent guid="guidSHLMainMenu" id="IDM_VS_CTXT_FOLDERNODE" />
			</Group>

			<Group guid="UnrealVSCmdSetGUID" id="ProjectUnrealVSUBTGroupID" priority="0x0000">
				<!-- Parent the menu group to the project node context menu -->
				<Parent guid="UnrealVSCmdSetGUID" id="ProjectUnrealVSUBTMenuID" />
			</Group>

			<Group guid="UnrealVSCmdSetGUID" id="FolderUnrealVSUBTGroupID" priority="0x0000">
				<!-- Parent the menu group to the project node context menu -->
				<Parent guid="UnrealVSCmdSetGUID" id="FolderUnrealVSUBTMenuID" />
			</Group>

			<Group guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildSubMenuGroupID" priority="0x0000">
				<Parent guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildSubMenuID" />
			</Group>

			<!-- Quick build menu groups -->
			<!-- Parent the submenus to the config specific menu group -->

			<!-- Debug -->
			<Group guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildDebugSubMenuGroupID" priority="0x0000">
				<Parent guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildDebugSubMenuID" />
			</Group>

			<Group guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildDebugClientSubMenuGroupID" priority="0x0000">
				<Parent guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildDebugClientSubMenuID" />
			</Group>

			<Group guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildDebugEditorSubMenuGroupID" priority="0x0000">
				<Parent guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildDebugEditorSubMenuID" />
			</Group>

			<Group guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildDebugServerSubMenuGroupID" priority="0x0000">
				<Parent guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildDebugServerSubMenuID" />
			</Group>

			<!-- DebugGame -->
			<Group guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildDebugGameSubMenuGroupID" priority="0x0000">
				<Parent guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildDebugGameSubMenuID" />
			</Group>

			<Group guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildDebugGameClientSubMenuGroupID" priority="0x0000">
				<Parent guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildDebugGameClientSubMenuID" />
			</Group>

			<Group guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildDebugGameEditorSubMenuGroupID" priority="0x0000">
				<Parent guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildDebugGameEditorSubMenuID" />
			</Group>

			<Group guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildDebugGameServerSubMenuGroupID" priority="0x0000">
				<Parent guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildDebugGameServerSubMenuID" />
			</Group>

			<!-- Development -->
			<Group guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildDevelopmentSubMenuGroupID" priority="0x0000">
				<Parent guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildDevelopmentSubMenuID" />
			</Group>

			<Group guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildDevelopmentClientSubMenuGroupID" priority="0x0000">
				<Parent guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildDevelopmentClientSubMenuID" />
			</Group>

			<Group guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildDevelopmentEditorSubMenuGroupID" priority="0x0000">
				<Parent guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildDevelopmentEditorSubMenuID" />
			</Group>

			<Group guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildDevelopmentServerSubMenuGroupID" priority="0x0000">
				<Parent guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildDevelopmentServerSubMenuID" />
			</Group>

			<!-- Shipping -->
			<Group guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildShippingSubMenuGroupID" priority="0x0000">
				<Parent guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildShippingSubMenuID" />
			</Group>

			<Group guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildShippingClientSubMenuGroupID" priority="0x0000">
				<Parent guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildShippingClientSubMenuID" />
			</Group>

			<Group guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildShippingEditorSubMenuGroupID" priority="0x0000">
				<Parent guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildShippingEditorSubMenuID" />
			</Group>

			<Group guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildShippingServerSubMenuGroupID" priority="0x0000">
				<Parent guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildShippingServerSubMenuID" />
			</Group>

			<!-- Test -->
			<Group guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildTestSubMenuGroupID" priority="0x0000">
				<Parent guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildTestSubMenuID" />
			</Group>

			<Group guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildTestClientSubMenuGroupID" priority="0x0000">
				<Parent guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildTestClientSubMenuID" />
			</Group>

			<Group guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildTestEditorSubMenuGroupID" priority="0x0000">
				<Parent guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildTestEditorSubMenuID" />
			</Group>

			<Group guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildTestServerSubMenuGroupID" priority="0x0000">
				<Parent guid="UnrealVSCmdSetGUID" id="ProjectQuickBuildTestServerSubMenuID" />
			</Group>

			<Group guid="UnrealVSCmdSetGUID" id="CopyIncludePathGroupID" priority="0x0600">
				<Parent guid="guidSHLMainMenu" id="IDM_VS_CTXT_EZDOCWINTAB"/>
			</Group>

			<Group guid="UnrealVSCmdSetGUID" id="P4SubMenuGroupContainer" priority="0x0">
				<Parent guid="guidSHLMainMenu" id="IDM_VS_CTXT_CODEWIN"/>
			</Group>
			<Group guid="UnrealVSCmdSetGUID" id="P4SubMenuGroup" priority="0x0">
				<Parent guid="UnrealVSCmdSetGUID" id="P4SubMenu" />
			</Group>
			<Group guid="UnrealVSCmdSetGUID" id="UBTSubMenuGroupContainer" priority="0x01">
				<Parent guid="guidSHLMainMenu" id="IDM_VS_CTXT_CODEWIN"/>
			</Group>
			<Group guid="UnrealVSCmdSetGUID" id="UBTSubMenuGroup" priority="0x01">
				<Parent guid="UnrealVSCmdSetGUID" id="UBTSubMenu" />
			</Group>
		</Groups>

		<Bitmaps>
			<Bitmap guid="RefreshProjectsIconGUID" href="Resources\RefreshProjects.png" />
			<Bitmap guid="BatchBuilderIconGUID" href="Resources\BatchBuilder.png" />
			<Bitmap guid="UnrealIconGUID" href="Resources\UnrealIcon.png" />
			<Bitmap guid="FileBrowserIconGUID" href="Resources\FileBrowser.png" />
			<Bitmap guid="BookmarkAddGUID" href="Resources\BookmarkAdd.png" />
			<Bitmap guid="BookmarkRemoveGUID" href="Resources\BookmarkRemove.png" />
			<Bitmap guid="CopyIncludeGUID" href="Resources\CopyInclude.png" />
			<Bitmap guid="InsertIncludeGUID" href="Resources\InsertInclude.png" />
			<Bitmap guid="CycleRelatedFilesIconGUID" href="Resources\CycleRelatedFiles.png" />
		</Bitmaps>

	</Commands>

	<!-- Add the P4SubMenu and UBTSubMenu to the Context Menus -->
	<CommandPlacements>
		<CommandPlacement guid="UnrealVSCmdSetGUID" id="P4SubMenu" priority="0x105">
			<Parent guid="UnrealVSCmdSetGUID" id="P4SubMenuGroupContainer" />
		</CommandPlacement>
		<CommandPlacement guid="UnrealVSCmdSetGUID" id="UBTSubMenu" priority="0x106">
			<Parent guid="UnrealVSCmdSetGUID" id="UBTSubMenuGroupContainer" />
		</CommandPlacement>
		<CommandPlacement guid="UnrealVSCmdSetGUID" id="FileBrowserButtonID" priority="0x107">
			<Parent guid="UnrealVSCmdSetGUID" id="UnrealVSToolbarGroupID" />
		</CommandPlacement>
		<CommandPlacement guid="UnrealVSCmdSetGUID" id="CycleRelatedFilesButtonID" priority="0x108">
			<Parent guid="UnrealVSCmdSetGUID" id="UnrealVSToolbarGroupID" />
		</CommandPlacement>
		<CommandPlacement guid="UnrealVSCmdSetGUID" id="GenerateProjectFilesButtonID" priority="0x109">
			<Parent guid="UnrealVSCmdSetGUID" id="UnrealVSToolbarGroupID" />
		</CommandPlacement>
		<CommandPlacement guid="UnrealVSCmdSetGUID" id="BatchBuilderToolWindowId" priority="0x10A">
			<Parent guid="UnrealVSCmdSetGUID" id="UnrealVSToolbarGroupID" />
		</CommandPlacement>
	</CommandPlacements>

	<!-- To define a binding, we have to specify the sequence of keys to press to activate the command (keystate).
		 A key binding can use one or two key and optionally up to two modifiers (Alt, Control or Shift keys); the
		 keys are specified using the key1 and key2 attributes (key2 can be omitted) and the modifiers using the
		 mod1 and mod2 attributes (both optional). -->
	<!-- NOTE:  Editor="guidVSStd97"  is a special GUID that means the keybinding should be available in the regular Visual Studio editor -->
	<!--<KeyBindings>
		<KeyBinding guid="UnrealVSCmdSetGUID" id="BuildStartupProjectButtonID" editor="guidVSStd97" key1="B" mod1="Control Alt Shift" />
	</KeyBindings>-->


	<Symbols>
		<!-- This is the package guid. -->
		<GuidSymbol name="UnrealVSPackageGUID" value="{ddbf523f-7eb6-4887-bd51-85a714ff87eb}" />

		<!-- This is the guid used to group the menu commands together -->
		<GuidSymbol name="UnrealVSCmdSetGUID" value="{f10be850-eeec-4cef-ac64-1bc2cbe0d447}">
			<IDSymbol name="UnrealVSToolbar" value="0x1010" />
			<IDSymbol name="UnrealVSToolbarGroupID" value="0x1020" />

			<IDSymbol name="CommandLineComboID" value="0x1030" />
			<IDSymbol name="CommandLineComboListID" value="0x1040" />

			<IDSymbol name="StartupProjectComboID" value="0x1050" />
			<IDSymbol name="StartupProjectComboListID" value="0x1060" />

			<IDSymbol name="BuildStartupProjectButtonID" value="0x1070" />
			<IDSymbol name="ProjectRebuildAndProfileButtonID" value="0x1071" />

			<IDSymbol name="FolderUnrealVSBuildButtonID" value="0x1170" />
			<IDSymbol name="FolderUnrealVSBuildNonUnityButtonID" value="0x1171" />
			<IDSymbol name="FolderUnrealVSRebuildAndProfileButtonID" value="0x1172" />
			<IDSymbol name="FolderUnrealVSRebuildAndProfileNonUnityButtonID" value="0x1173" />

			<IDSymbol name="CompileSingleFileButtonID" value="0x1075" />
			<IDSymbol name="PreprocessSingleFileButtonID" value="0x1076" />
			<IDSymbol name="CompileSingleModuleButtonID" value="0x1077" />
			<IDSymbol name="CompileAndProfileSingleFileButtonID" value="0x1078" />
			<IDSymbol name="GenerateAssemblyFileButtonID" value="0x1079" />

			<IDSymbol name="FileBrowserButtonID" value="0x1337" />
			<IDSymbol name="UbaVisualizerButtonID" value="0x1338" />
			<IDSymbol name="CycleRelatedFilesButtonID" value="0x1339" />

			<IDSymbol name="CommandsMenuID" value="0x1080" />
			<IDSymbol name="CommandsMenuGroupID" value="0x1090" />

			<IDSymbol name="GenerateProjectFilesButtonID" value="0x1100" />
			<IDSymbol name="BatchBuilderToolWindowId" value="0x1300" />

			<IDSymbol name="UnrealVSProjectGroupID" value="0x1400" />
			<IDSymbol name="ProjectUnrealVSUBTGroupID" value="0x1401" />
			<IDSymbol name="ProjectUnrealVSUBTMenuID" value="0x1402" />
			<IDSymbol name="ProjectQuickBuildSubMenuID" value="0x1410" />
			<IDSymbol name="ProjectQuickBuildSubMenuGroupID" value="0x1420" />
			
			<IDSymbol name="UnrealVSFolderGroupID" value="0x1430" />
			<IDSymbol name="FolderUnrealVSUBTMenuID" value="0x1431" />
			<IDSymbol name="FolderUnrealVSUBTGroupID" value="0x1432" />

			<IDSymbol name="CopyIncludePathButtonID" value="0x1470" />
			<IDSymbol name="CopyIncludePathGroupID" value="0x1471" />

			<IDSymbol name="P4SubMenu" value="0x3100" />
			<IDSymbol name="P4SubMenuGroup" value="0x3101" />
			<IDSymbol name="P4SubMenuGroupContainer" value="0x3102" />
			<IDSymbol name="UBTSubMenu" value="0x3103" />
			<IDSymbol name="UBTSubMenuGroup" value="0x3104" />
			<IDSymbol name="UBTSubMenuGroupContainer" value="0x3105" />

			<IDSymbol name="P4CheckoutButtonID" value="0x1450" />
			<IDSymbol name="P4AnnotateButtonID" value="0x1451" />
			<IDSymbol name="P4ViewSelectedCLButtonID" value="0x1452" />
			<IDSymbol name="P4IntegrationAwareTimelapseButtonID" value="0x1453" />
			<IDSymbol name="P4DiffButtonID" value="0x1454" />
			<IDSymbol name="P4GetLast10ChangesID" value="0x1455" />
			<IDSymbol name="P4ShowFileinP4VID" value="0x1456" />
			<IDSymbol name="P4FastReconcileCodeFiles" value="0x1457" />
			<IDSymbol name="P4TimelapseButtonID" value="0x1458" />
			<IDSymbol name="P4RevisionGraphButtonID" value="0x1459" />
			<IDSymbol name="P4FileHistoryButtonID" value="0x1460" />
			<IDSymbol name="P4RevertButtonID" value="0x1461" />
			<!-- Quickbuild configuration menus -->
			<!-- Debug -->
			<IDSymbol name="ProjectQuickBuildDebugSubMenuID" value="0x1500" />
			<IDSymbol name="ProjectQuickBuildDebugSubMenuGroupID" value="0x1510" />
			<IDSymbol name="ProjectQuickBuildDebugDynamicStartCommandID" value="0x1530" />

			<IDSymbol name="ProjectQuickBuildDebugClientSubMenuID" value="0x1600" />
			<IDSymbol name="ProjectQuickBuildDebugClientSubMenuGroupID" value="0x1610" />
			<IDSymbol name="ProjectQuickBuildDebugClientDynamicStartCommandID" value="0x1630" />

			<IDSymbol name="ProjectQuickBuildDebugEditorSubMenuID" value="0x1700" />
			<IDSymbol name="ProjectQuickBuildDebugEditorSubMenuGroupID" value="0x1710" />
			<IDSymbol name="ProjectQuickBuildDebugEditorDynamicStartCommandID" value="0x1730" />

			<IDSymbol name="ProjectQuickBuildDebugServerSubMenuID" value="0x1800" />
			<IDSymbol name="ProjectQuickBuildDebugServerSubMenuGroupID" value="0x1810" />
			<IDSymbol name="ProjectQuickBuildDebugServerDynamicStartCommandID" value="0x1830" />

			<!-- DebugGame -->
			<IDSymbol name="ProjectQuickBuildDebugGameSubMenuID" value="0x1900" />
			<IDSymbol name="ProjectQuickBuildDebugGameSubMenuGroupID" value="0x1910" />
			<IDSymbol name="ProjectQuickBuildDebugGameDynamicStartCommandID" value="0x1930" />

			<IDSymbol name="ProjectQuickBuildDebugGameClientSubMenuID" value="0x1A00" />
			<IDSymbol name="ProjectQuickBuildDebugGameClientSubMenuGroupID" value="0x1A10" />
			<IDSymbol name="ProjectQuickBuildDebugGameClientDynamicStartCommandID" value="0x1A30" />

			<IDSymbol name="ProjectQuickBuildDebugGameEditorSubMenuID" value="0x1B00" />
			<IDSymbol name="ProjectQuickBuildDebugGameEditorSubMenuGroupID" value="0x1B10" />
			<IDSymbol name="ProjectQuickBuildDebugGameEditorDynamicStartCommandID" value="0x1B30" />

			<IDSymbol name="ProjectQuickBuildDebugGameServerSubMenuID" value="0x1C00" />
			<IDSymbol name="ProjectQuickBuildDebugGameServerSubMenuGroupID" value="0x1C10" />
			<IDSymbol name="ProjectQuickBuildDebugGameServerDynamicStartCommandID" value="0x1C30" />

			<!-- Development -->
			<IDSymbol name="ProjectQuickBuildDevelopmentSubMenuID" value="0x1D00" />
			<IDSymbol name="ProjectQuickBuildDevelopmentSubMenuGroupID" value="0x1D10" />
			<IDSymbol name="ProjectQuickBuildDevelopmentDynamicStartCommandID" value="0x1D30" />

			<IDSymbol name="ProjectQuickBuildDevelopmentClientSubMenuID" value="0x1E00" />
			<IDSymbol name="ProjectQuickBuildDevelopmentClientSubMenuGroupID" value="0x1E10" />
			<IDSymbol name="ProjectQuickBuildDevelopmentClientDynamicStartCommandID" value="0x1E30" />

			<IDSymbol name="ProjectQuickBuildDevelopmentEditorSubMenuID" value="0x1F00" />
			<IDSymbol name="ProjectQuickBuildDevelopmentEditorSubMenuGroupID" value="0x1F10" />
			<IDSymbol name="ProjectQuickBuildDevelopmentEditorDynamicStartCommandID" value="0x1F30" />

			<IDSymbol name="ProjectQuickBuildDevelopmentServerSubMenuID" value="0x2000" />
			<IDSymbol name="ProjectQuickBuildDevelopmentServerSubMenuGroupID" value="0x2010" />
			<IDSymbol name="ProjectQuickBuildDevelopmentServerDynamicStartCommandID" value="0x2030" />

			<!-- Shipping -->
			<IDSymbol name="ProjectQuickBuildShippingSubMenuID" value="0x2100" />
			<IDSymbol name="ProjectQuickBuildShippingSubMenuGroupID" value="0x2110" />
			<IDSymbol name="ProjectQuickBuildShippingDynamicStartCommandID" value="0x2130" />

			<IDSymbol name="ProjectQuickBuildShippingClientSubMenuID" value="0x2200" />
			<IDSymbol name="ProjectQuickBuildShippingClientSubMenuGroupID" value="0x2210" />
			<IDSymbol name="ProjectQuickBuildShippingClientDynamicStartCommandID" value="0x2230" />

			<IDSymbol name="ProjectQuickBuildShippingEditorSubMenuID" value="0x2300" />
			<IDSymbol name="ProjectQuickBuildShippingEditorSubMenuGroupID" value="0x2310" />
			<IDSymbol name="ProjectQuickBuildShippingEditorDynamicStartCommandID" value="0x2330" />

			<IDSymbol name="ProjectQuickBuildShippingServerSubMenuID" value="0x2400" />
			<IDSymbol name="ProjectQuickBuildShippingServerSubMenuGroupID" value="0x2410" />
			<IDSymbol name="ProjectQuickBuildShippingServerDynamicStartCommandID" value="0x2430" />

			<!-- Test -->
			<IDSymbol name="ProjectQuickBuildTestSubMenuID" value="0x2500" />
			<IDSymbol name="ProjectQuickBuildTestSubMenuGroupID" value="0x2510" />
			<IDSymbol name="ProjectQuickBuildTestDynamicStartCommandID" value="0x2530" />

			<IDSymbol name="ProjectQuickBuildTestClientSubMenuID" value="0x2600" />
			<IDSymbol name="ProjectQuickBuildTestClientSubMenuGroupID" value="0x2610" />
			<IDSymbol name="ProjectQuickBuildTestClientDynamicStartCommandID" value="0x2630" />

			<IDSymbol name="ProjectQuickBuildTestEditorSubMenuID" value="0x2700" />
			<IDSymbol name="ProjectQuickBuildTestEditorSubMenuGroupID" value="0x2710" />
			<IDSymbol name="ProjectQuickBuildTestEditorDynamicStartCommandID" value="0x2730" />

			<IDSymbol name="ProjectQuickBuildTestServerSubMenuID" value="0x2800" />
			<IDSymbol name="ProjectQuickBuildTestServerSubMenuGroupID" value="0x2810" />
			<IDSymbol name="ProjectQuickBuildTestServerDynamicStartCommandID" value="0x2830" />

		</GuidSymbol>

		<GuidSymbol name="RefreshProjectsIconGUID" value="{8F4278B0-4C7D-4B67-9C27-AE415543EBE5}">
			<IDSymbol name="RefreshProjectsIcon" value="1" />
		</GuidSymbol>

		<GuidSymbol name="BatchBuilderIconGUID" value="{4EA0D612-B5D9-4D49-A41E-9031B0A37D12}">
			<IDSymbol name="BatchBuilderIcon" value="1" />
		</GuidSymbol>

		<GuidSymbol name="FileBrowserIconGUID" value="{89F71288-13B6-46FF-A9D3-1EDC712AC0FC}">
			<IDSymbol name="FileBrowserIcon" value="1" />
		</GuidSymbol>

		<GuidSymbol name="BookmarkAddGUID" value="{89F71288-13B6-46FF-A9D3-1EDC712AC0FD}">
			<IDSymbol name="BookmarkAddIcon" value="1" />
		</GuidSymbol>

		<GuidSymbol name="BookmarkRemoveGUID" value="{89F71288-13B6-46FF-A9D3-1EDC712AC0FE}">
			<IDSymbol name="BookmarkRemoveIcon" value="1" />
		</GuidSymbol>

		<GuidSymbol name="CopyIncludeGUID" value="{89F71288-13B6-46FF-A9D3-1EDC712AC0FF}">
			<IDSymbol name="CopyIncludeIcon" value="1" />
		</GuidSymbol>

		<GuidSymbol name="InsertIncludeGUID" value="{89F71288-13B6-46FF-A9D3-1EDC712AC0AC}">
			<IDSymbol name="InsertIncludeIcon" value="1" />
		</GuidSymbol>

		<GuidSymbol name="CycleRelatedFilesIconGUID" value="{590283A7-D6EA-463A-98C6-F0FE4A42AB66}">
			<IDSymbol name="CycleRelatedFilesIcon" value="1" />
		</GuidSymbol>

		<GuidSymbol name="UnrealIconGUID" value="{7B2A8E4F-3D1C-4A9E-B5F6-8C0D2E4F6A1B}">
			<IDSymbol name="UnrealIcon" value="1" />
		</GuidSymbol>

	</Symbols>

</CommandTable>
