<?xml version="1.0" encoding="utf-8"?>
<PackageManifest Version="2.0.0" xmlns="http://schemas.microsoft.com/developer/vsx-schema/2011" xmlns:d="http://schemas.microsoft.com/developer/vsx-schema-design/2011">
    <Metadata>
        <Identity Id="ddbf523f-7eb6-4887-bd51-85a714ff87eb" Version="1.87.0.0" Language="en-US" Publisher="Epic Games, Inc." />
        <DisplayName>UnrealVS</DisplayName>
        <Description xml:space="preserve">Unreal Engine extension for Visual Studio</Description>
        <MoreInfo>https://docs.unrealengine.com/en-US/ProductionPipelines/DevelopmentSetup/VisualStudioSetup/UnrealVS/index.html</MoreInfo>
        <ReleaseNotes>Resources\ReleaseNotes.txt</ReleaseNotes>
        <Icon>Resources\Icon.png</Icon>
        <PreviewImage>Resources\Preview.png</PreviewImage>
    </Metadata>
    <Installation>
        <InstallationTarget Id="Microsoft.VisualStudio.Pro" Version="[17.0,18.0)">
            <ProductArchitecture>amd64</ProductArchitecture>
        </InstallationTarget>
        <InstallationTarget Id="Microsoft.VisualStudio.Community" Version="[17.0,18.0)">
            <ProductArchitecture>amd64</ProductArchitecture>
        </InstallationTarget>
        <InstallationTarget Id="Microsoft.VisualStudio.Enterprise" Version="[17.0,18.0)">
            <ProductArchitecture>amd64</ProductArchitecture>
        </InstallationTarget>
        <InstallationTarget Id="Microsoft.VisualStudio.Pro" Version="[17.0,18.0)">
            <ProductArchitecture>arm64</ProductArchitecture>
        </InstallationTarget>
        <InstallationTarget Id="Microsoft.VisualStudio.Community" Version="[17.0,18.0)">
            <ProductArchitecture>arm64</ProductArchitecture>
        </InstallationTarget>
        <InstallationTarget Id="Microsoft.VisualStudio.Enterprise" Version="[17.0,18.0)">
            <ProductArchitecture>arm64</ProductArchitecture>
        </InstallationTarget>
    </Installation>
    <Dependencies>
        <Dependency Id="Microsoft.Framework.NDP" DisplayName="Microsoft .NET Framework" d:Source="Manual" Version="[4.7.2,)" />
    </Dependencies>
    <Assets>
        <Asset Type="Microsoft.VisualStudio.VsPackage" d:Source="Project" d:ProjectName="%CurrentProject%" Path="|%CurrentProject%;PkgdefProjectOutputGroup|" />
        <Asset Type="UnitTestExtension" d:Source="Project" d:ProjectName="Unreal.TestAdapter" Path="|Unreal.TestAdapter|" />
    </Assets>
    <Prerequisites>
        <Prerequisite Id="Microsoft.VisualStudio.Component.CoreEditor" Version="[17.0,18.0)" DisplayName="Visual Studio core editor" />
    </Prerequisites>
</PackageManifest>