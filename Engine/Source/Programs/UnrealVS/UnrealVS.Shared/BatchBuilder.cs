// Copyright Epic Games, Inc. All Rights Reserved.

using Microsoft.VisualStudio.Shell;
using Microsoft.VisualStudio.Shell.Interop;
using System;
using System.ComponentModel.Design;
using System.IO;

namespace UnrealVS
{
	internal class BatchBuilder
	{
		/** constants */

		private const int BatchBuilderToolWindowId = 0x1300;

		/** properties */

		public bool IsBusy
		{
			get
			{
				return ToolControl.IsBusy;
			}
		}

		public static BatchBuilderToolControl.BatchBuilderToolState _ToolControlState = new BatchBuilderToolControl.BatchBuilderToolState();
		private static BatchBuilderToolControl _ToolControl;
		public static BatchBuilderToolControl ToolControl
		{
			get
			{
				if (_ToolControl == null) _ToolControl = new BatchBuilderToolControl(_ToolControlState);
				return _ToolControl;
			}
		}

		/** methods */

		public BatchBuilder()
		{
		}

		public void Initialize()
		{
			// Create the command for the tool window
			var ToolWindowCommandId = new CommandID(GuidList.UnrealVSCmdSet, BatchBuilderToolWindowId);
			var ToolWindowMenuCommand = new MenuCommand(ShowToolWindow, ToolWindowCommandId);
			UnrealVSPackage.Instance.MenuCommandService.AddCommand(ToolWindowMenuCommand);
		}

		/// <summary>
		/// Called from the package class when there are options to be read out of the solution file.
		/// </summary>
		/// <param name="Stream">The stream to load the option data from.</param>
		public void LoadOptions(Stream Stream)
		{
			_ToolControlState.LoadOptions(Stream);
		}

		/// <summary>
		/// Called from the package class when there are options to be written to the solution file.
		/// </summary>
		/// <param name="Stream">The stream to save the option data to.</param>
		public void SaveOptions(Stream Stream)
		{
			_ToolControlState.SaveOptions(Stream);
		}

		/// <summary>
		/// Tick function called from the package
		/// </summary>
		public void Tick()
		{
			ThreadHelper.ThrowIfNotOnUIThread();
			ToolControl.Tick();
		}

		private void ShowToolWindow(object sender, EventArgs e)
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			// Get the instance number 0 of this tool window. This window is single instance so this instance
			// is actually the only one.
			// The last flag is set to true so that if the tool window does not exists it will be created.
			ToolWindowPane ToolWindow = UnrealVSPackage.Instance.FindToolWindow(typeof(BatchBuilderToolWindow), 0, true);
			if ((null == ToolWindow) || (null == ToolWindow.Frame))
			{
				throw new NotSupportedException(Resources.ToolWindowCreateError);
			}
			IVsWindowFrame ToolWindowFrame = (IVsWindowFrame)ToolWindow.Frame;
			Microsoft.VisualStudio.ErrorHandler.ThrowOnFailure(ToolWindowFrame.Show());
		}
	}
}
