<!--Copyright Epic Games, Inc. All Rights Reserved.-->

<UserControl x:Class="UnrealVS.BatchBuilderToolControl"
			 xmlns="http://schemas.microsoft.com/winfx/2006/xaml/presentation"
			 xmlns:x="http://schemas.microsoft.com/winfx/2006/xaml"
			 xmlns:mc="http://schemas.openxmlformats.org/markup-compatibility/2006"
			 xmlns:d="http://schemas.microsoft.com/expression/blend/2008"
			 xmlns:vs_shell="clr-namespace:Microsoft.VisualStudio.PlatformUI;assembly=Microsoft.VisualStudio.Shell.15.0"
			 xmlns:UnrealVS="clr-namespace:UnrealVS"
			 mc:Ignorable="d" d:DesignHeight="250" d:DesignWidth="800"
			 MinHeight="200" MinWidth="600" Name="ThisBatchBuilderToolCtrl"
						 DataContext="{Binding RelativeSource={RelativeSource Self}}"
						 IsEnabled="{Binding Path=IsSolutionOpen}">

	<UserControl.CommandBindings>
		<CommandBinding Command="Open" Executed="OpenItemOutputCommandExecuted"/>
	</UserControl.CommandBindings>

	<UserControl.Resources>

		<!-- Style for all ListBoxItems -->
		<Style TargetType="ListBoxItem">
			<Setter Property="Template">
				<Setter.Value>
					<ControlTemplate TargetType="ListBoxItem">
						<Border Name="Border">
							<ContentPresenter />
						</Border>
						<ControlTemplate.Triggers>
							<MultiTrigger>
								<MultiTrigger.Conditions>
									<Condition Property="IsSelected" Value="false"/>
									<Condition Property="IsEnabled" Value="true"/>
								</MultiTrigger.Conditions>
								<Setter Property="Foreground" Value="{x:Static SystemColors.WindowTextBrush}" />
								<Setter TargetName="Border" Property="Background" Value="{x:Static SystemColors.WindowBrush}" />
							</MultiTrigger>
							<Trigger Property="IsSelected" Value="true">
								<Setter Property="Foreground" Value="{x:Static SystemColors.HighlightTextBrush}" />
								<Setter TargetName="Border" Property="Background"
										Value="{x:Static SystemColors.HighlightBrush}" />
							</Trigger>
						</ControlTemplate.Triggers>
					</ControlTemplate>
				</Setter.Value>
			</Setter>
		</Style>

		<!-- DataTemplate ProjectListItemTemplate -->
		<DataTemplate x:Key="ProjectListItemTemplate">
			<TextBlock Text="{Binding}" />
		</DataTemplate>

		<!-- DataTemplate BuildListItemTemplate -->
		<DataTemplate x:Key="BuildListItemTemplate">
			<ContentControl MouseDoubleClick="OnDblClickBuildListItem" ToolTip="Double-click to switch the active solution config and the startup project to match this item.">
				<TextBlock Text="{Binding Path=DisplayString}"/>
			</ContentControl>
		</DataTemplate>

		<!-- DataTemplate BuildingListItemTemplate -->
		<DataTemplate x:Key="BuildingListItemTemplate">
			<ContentControl MouseDoubleClick="OnDblClickBuildingListItem">
				<StackPanel Orientation="Horizontal">
					<StackPanel.Style>
						<Style>
							<Style.Triggers>
								<DataTrigger Binding="{Binding Path=HasOutputText}" Value="True">
									<Setter Property="FrameworkElement.ContextMenu">
										<Setter.Value>
											<ContextMenu>
												<MenuItem Header="Show Output" Command="Open" CommandParameter="{Binding}"/>
											</ContextMenu>
										</Setter.Value>
									</Setter>
								</DataTrigger>
							</Style.Triggers>
						</Style>
					</StackPanel.Style>
					<UnrealVS:BuildStatusIcon Margin="0,2"  Width="15" Height="15" BuildStatus="{Binding Path=JobStatus}" ToolTip="{Binding Path=JobStatusDisplayString}"/>
					<TextBlock VerticalAlignment="Center" Margin="5,0,0,0" Text="{Binding Path=DisplayString}" ToolTip="{Binding Path=DisplayString}">
						<TextBlock.Style>
							<Style>
								<Style.Triggers>
									<DataTrigger Binding="{Binding Path=JobStatus}" Value="Executing">
										<Setter Property="TextBlock.FontWeight" Value="Bold"/>
									</DataTrigger>
								</Style.Triggers>
								<Setter Property="TextBlock.FontWeight" Value="Normal"/>
							</Style>
						</TextBlock.Style>
					</TextBlock>
				</StackPanel>
			</ContentControl>
		</DataTemplate>

		<!-- Style DisabledOnBusy -->
		<Style x:Key="DisabledOnBusy">
			<Style.Triggers>
				<DataTrigger Binding="{Binding Path=IsBusy}" Value="True">
					<Setter Property="UIElement.IsEnabled" Value="False"/>
				</DataTrigger>
			</Style.Triggers>
			<Setter Property="UIElement.IsEnabled" Value="True"/>
		</Style>

		<!-- Style ShowOnBusy -->
		<Style x:Key="ShowOnBusy">
			<Style.Triggers>
				<DataTrigger Binding="{Binding Path=IsBusy}" Value="False">
					<Setter Property="UIElement.Visibility" Value="Collapsed"/>
				</DataTrigger>
			</Style.Triggers>
			<Setter Property="UIElement.Visibility" Value="Visible"/>
		</Style>

		<!-- Style BuildJobsListStyle -->
		<Style x:Key="BuildJobsListStyle">
			<Style.Triggers>
				<DataTrigger Binding="{Binding Path=IsBusy}" Value="True">
					<!--<Setter Property="ListBox.ItemTemplate" Value="{StaticResource NullItemTemplate}"/>-->
					<Setter Property="ListBox.IsEnabled" Value="False"/>
				</DataTrigger>
			</Style.Triggers>
			<Setter Property="ListBox.IsEnabled" Value="True"/>
			<Setter Property="ListBox.ItemTemplate" Value="{DynamicResource BuildListItemTemplate}"/>
			<Setter Property="ListBox.SelectionMode" Value="Extended"/>
			<EventSetter Event="ListBox.SelectionChanged" Handler="OnBuildJobsSelectionChanged"/>
		</Style>

		<!-- Style BuildingJobsListStyle -->
		<Style x:Key="BuildingJobsListStyle">
			<Setter Property="ItemsControl.Background" Value="Transparent"/>
			<Setter Property="ItemsControl.ItemTemplate" Value="{DynamicResource BuildingListItemTemplate}"/>
			<Setter Property="ItemsControl.Margin" Value="3"/>
		</Style>

		<!-- Style OutputTabStyle -->
		<Style x:Key="OutputTabStyle">
			<Style.Triggers>
				<DataTrigger Binding="{Binding Path=HasOutput}" Value="True">
					<Setter Property="TabItem.Visibility" Value="Visible"/>
				</DataTrigger>
				<DataTrigger Binding="{Binding Path=HasOutput}" Value="False">
					<Setter Property="TabItem.Visibility" Value="Collapsed"/>
				</DataTrigger>
			</Style.Triggers>
		</Style>

		<!-- Style StartButtonStyle -->
		<Style x:Key="StartButtonStyle" TargetType="{x:Type Button}">
			<Style.Triggers>
				<DataTrigger Binding="{Binding Path=IsBusy}" Value="True">
					<Setter Property="Content" Value="Stop"/>
				</DataTrigger>
			</Style.Triggers>
			<Setter Property="Content" Value="Start"/>
			<Setter Property="Margin" Value="0, 5, 0, 0"/>
			<Setter Property="Width" Value="45"/>
			<Setter Property="ToolTip" Value="Start/Stop build jobs in the list"/>
			<EventSetter Event="Click" Handler="OnStartStopButtonClick"/>
		</Style>

		<!-- Style StopButtonStyle -->
		<Style x:Key="StopButtonStyle" TargetType="{x:Type Button}">
			<Style.Triggers>
				<DataTrigger Binding="{Binding Path=IsBusy}" Value="False">
					<Setter Property="IsEnabled" Value="False"/>
				</DataTrigger>
			</Style.Triggers>
			<Setter Property="IsEnabled" Value="True"/>
			<Setter Property="Content" Value="Stop"/>
			<Setter Property="Margin" Value="0, 5, 0, 0"/>
			<Setter Property="Width" Value="45"/>
			<Setter Property="ToolTip" Value="Stop/cancel running build jobs"/>
			<EventSetter Event="Click" Handler="OnStartStopButtonClick"/>
		</Style>

		<Style x:Key="LabelStyle" TargetType="{x:Type TextBlock}">
			<Setter Property="Foreground" Value="{DynamicResource {x:Static vs_shell:EnvironmentColors.ToolWindowTextBrushKey}}"/>
			<Setter Property="Background" Value="{DynamicResource {x:Static vs_shell:EnvironmentColors.ToolWindowBackgroundBrushKey}}"/>
		</Style>

	</UserControl.Resources>

	<!-- CONTENT -->

	<Grid Margin="3">
		<Grid.ColumnDefinitions>
			<ColumnDefinition Width="4*"/>
			<ColumnDefinition Width="6"/>
			<ColumnDefinition Width="3*"/>
			<ColumnDefinition Width="6"/>
			<ColumnDefinition Width="3*"/>
			<ColumnDefinition Width="6"/>
			<ColumnDefinition Width="7*"/>
		</Grid.ColumnDefinitions>
		<DockPanel Margin="0,3" Grid.Column="0">
			<TextBlock Style="{StaticResource LabelStyle}" Margin="3,0,3,3" DockPanel.Dock="Top">Project</TextBlock>
			<ListBox Name="ProjectsList" ItemsSource="{Binding Path=Projects}"
					 SelectionMode="Extended" ItemTemplate="{DynamicResource ProjectListItemTemplate}" />
		</DockPanel>
		<GridSplitter Grid.Column="1" HorizontalAlignment="Stretch" Margin="1,22,1,3"/>
		<DockPanel Margin="0,3" Grid.Column="2">
			<TextBlock Style="{StaticResource LabelStyle}" Margin="3,0,3,3" DockPanel.Dock="Top">Config</TextBlock>
			<ListBox Name="ConfigsList" ItemsSource="{Binding Path=Configs}"
					 SelectionMode="Extended" />
		</DockPanel>
		<GridSplitter Grid.Column="3" HorizontalAlignment="Stretch" Margin="1,22,1,3"/>
		<Image Margin="20,0,0,-28" Height="150" Width="150" HorizontalAlignment="Left" VerticalAlignment="Bottom" Grid.Column="4" Grid.ColumnSpan="3" Source="Resources/BatchBuilderBackground.png">
			<Image.BitmapEffect>
				<BlurBitmapEffect Radius="7.5"/>
			</Image.BitmapEffect>
		</Image>
		<DockPanel Margin="0,3" Grid.Column="4">
			<TextBlock Style="{StaticResource LabelStyle}" Margin="3,0,3,3" DockPanel.Dock="Top">Platform</TextBlock>
			<StackPanel DockPanel.Dock="Bottom" HorizontalAlignment="Stretch" Background="#70ffffff" Margin="3,3,3,0">
				<RadioButton Margin="3" Name="BuildRadioButton">Build</RadioButton>
				<RadioButton Margin="3" Name="RebuildRadioButton">Rebuild</RadioButton>
				<RadioButton Margin="3" Name="CleanRadioButton">Clean</RadioButton>
			</StackPanel>
			<ListBox Name="PlatformsList" ItemsSource="{Binding Path=Platforms}"
					 SelectionMode="Extended" />
		</DockPanel>
		<GridSplitter Grid.Column="5" HorizontalAlignment="Stretch" Margin="1,22,1,3"/>
		<DockPanel Grid.Column="6">
			<StackPanel DockPanel.Dock="Left" VerticalAlignment="Center" Grid.Column="6" Background="#70ffffff" Margin="2">
				<Button Style="{StaticResource DisabledOnBusy}" Margin="2,2,2,0" Width="24" Height="24" FontSize="15" Click="AddButtonClick" ToolTip="Adds Selected Projects, Configs and Platforms to the Build Jobs List">&gt;</Button>
				<Button Style="{StaticResource DisabledOnBusy}" Margin="2" Width="24" Height="24" FontSize="15" Click="RemoveButtonClick" ToolTip="Removes Selected Build Jobs from the Build Jobs List">&lt;</Button>
			</StackPanel>
			<TabControl Margin="0,0,3,3" Grid.Column="7">
				<TabItem Name="JobsListTab" Header="{Binding BuildJobsPanelTitle}">
					<Grid>
						<Grid.ColumnDefinitions>
							<ColumnDefinition Width="*"/>
							<ColumnDefinition Width="28"/>
						</Grid.ColumnDefinitions>

						<Grid.RowDefinitions>
							<RowDefinition Height="*"/>
							<RowDefinition Height="Auto"/>
						</Grid.RowDefinitions>

						<StackPanel Grid.Row="0" Margin="2,0" Grid.Column="1" Style="{StaticResource DisabledOnBusy}" HorizontalAlignment="Stretch">
							<Button Margin="0" FontSize="15" Name="UpButton" ToolTip="Move Up" Click="UpButtonClick" IsEnabled="{Binding Path=IsSingleBuildJobSelected}">&#x2191;</Button>
							<Button Margin="0,5,0,0" FontSize="15" Name="DownButton" ToolTip="Move Down" Click="DownButtonClick" IsEnabled="{Binding Path=IsSingleBuildJobSelected}">&#x2193;</Button>
						</StackPanel>

						<Grid Grid.Column="0" Grid.ColumnSpan="2" Grid.Row="1">
							<Grid.ColumnDefinitions>
								<ColumnDefinition Width="*"/>
								<ColumnDefinition Width="Auto"/>
							</Grid.ColumnDefinitions>

							<GroupBox Grid.Column="0" Style="{StaticResource DisabledOnBusy}" Header="Job Set">
								<DockPanel>
									<Button DockPanel.Dock="Right" Margin="5,0,0,0" Height="22" Name="DeleteButton" Click="OnDeleteButtonClick" IsEnabled="{Binding IsDeletableSetSelected}" ToolTip="Deletes the selected Job Set">Delete</Button>
									<ComboBox Name="SetCombo" IsEditable="True" Text="{Binding Path=SelectedBuildJobSetName, Mode=OneWayToSource, UpdateSourceTrigger=LostFocus}" ItemsSource="{Binding Path=BuildJobSets}" SelectionChanged="OnSetComboSelectionChanged" KeyDown="OnSetComboKeyDown"/>
								</DockPanel>
							</GroupBox>

							<Grid Grid.Column="1" Margin="3,0,0,0">
								<Grid.RowDefinitions>
									<RowDefinition Height="13"/>
									<RowDefinition Height="*"/>
								</Grid.RowDefinitions>

								<ProgressBar Grid.Row="0"
										 Style="{StaticResource ShowOnBusy}" Margin="2" 
										 IsIndeterminate="True" Foreground="#FF007ACC"
										 Width="{Binding ActualWidth, ElementName=StartStopButton}"/>
								<Button Grid.Row="1" Name="StartStopButton" Height="22"
									VerticalAlignment="Top" 
									Style="{StaticResource StartButtonStyle}"/>
							</Grid>

						</Grid>

						<ListBox Name="BuildJobsList" Style="{StaticResource BuildJobsListStyle}">
							<ListBox.ItemsSource>
								<Binding Path="BuildJobs" />
							</ListBox.ItemsSource>
						</ListBox>
					</Grid>
				</TabItem>
				<TabItem Name="OutputTab" Header="{Binding OutputPanelTitle}" Style="{StaticResource OutputTabStyle}">
					<DockPanel>
						<DockPanel DockPanel.Dock="Bottom">
							<StackPanel Grid.Column="1" HorizontalAlignment="Right">
								<ProgressBar Style="{StaticResource ShowOnBusy}" Height="10" Margin="0, 5, 0, 0" IsIndeterminate="True" Foreground="#FF007ACC"/>
								<Button Name="StopButton" Style="{StaticResource StopButtonStyle}"/>
							</StackPanel>
						</DockPanel>
						<ScrollViewer VerticalScrollBarVisibility="Auto">
							<ItemsControl Name="BuildingJobsList" Style="{StaticResource BuildingJobsListStyle}">
								<ItemsControl.ItemsSource>
									<Binding Path="LastBuildJobsQueued" />
								</ItemsControl.ItemsSource>
							</ItemsControl>
						</ScrollViewer>
					</DockPanel>
				</TabItem>
			</TabControl>
		</DockPanel>
	</Grid>

	<!-- END CONTENT -->

</UserControl>