// Copyright Epic Games, Inc. All Rights Reserved.

using EnvDTE;
using EnvDTE80;
using Microsoft.VisualStudio;
using Microsoft.VisualStudio.Shell;
using Microsoft.VisualStudio.Shell.Interop;
using System;
using System.Collections.Generic;
using System.Collections.ObjectModel;
using System.IO;
using System.Linq;
using System.Text;
using System.Windows;
using System.Windows.Controls;
using System.Windows.Input;
using Constants = EnvDTE.Constants;

namespace UnrealVS
{
	/// <summary>
	/// Interaction logic for BatchBuilderToolControl.xaml
	/// </summary>
	public partial class BatchBuilderToolControl : UserControl
	{
		private readonly BatchBuilderToolState _state;
		////////////// Types ///////////////////////

		public class BuildJob : DependencyObject
		{
			////////////// BuildJob Types ///////////////////////

			public enum BuildJobType
			{
				Build,
				Rebuild,
				Clean
			}

			public enum BuildJobStatus
			{
				Pending,
				Executing,
				Failed,
				FailedToStart,
				Cancelled,
				Complete
			}

			////////////// BuildJob Serialized fields/properties ///////////////////////

			private readonly Utils.SafeProjectReference _Project;
			private readonly string _Config;
			private readonly string _Platform;
			private readonly BuildJobType _JobType;

			////////////// BuildJob Properties ///////////////////////

			public Utils.SafeProjectReference Project
			{
				get { return _Project; }
			}

			public string Config
			{
				get { return _Config; }
			}

			public string Platform
			{
				get { return _Platform; }
			}

			public BuildJobType JobType
			{
				get { return _JobType; }
			}

			public TimeSpan ExecutionTime { get; set; }

			private readonly string _SolutionConfigString;
			public string SolutionConfigString
			{
				get { return _SolutionConfigString; }
			}

			private readonly string _DisplayString;
			public string DisplayString
			{
				get { return _DisplayString; }
			}

			////////////// BuildJob Dependency properties ///////////////////////

			public static readonly DependencyProperty JobStatusProperty;
			public static readonly DependencyProperty JobStatusDisplayStringProperty;
			public static readonly DependencyProperty OutputTextProperty;
			public static readonly DependencyProperty HasOutputTextProperty;

			////////////// BuildJob Dependency property wrappers ///////////////////////

			public BuildJobStatus JobStatus
			{
				get { return (BuildJobStatus)GetValue(JobStatusProperty); }
				set { SetValue(JobStatusProperty, value); }
			}

			public string JobStatusDisplayString
			{
				get { return (string)GetValue(JobStatusDisplayStringProperty); }
				private set { SetValue(JobStatusDisplayStringProperty, value); }
			}

			public string OutputText
			{
				get { return (string)GetValue(OutputTextProperty); }
				set { SetValue(OutputTextProperty, value); }
			}

			public bool HasOutputText
			{
				get { return (bool)GetValue(HasOutputTextProperty); }
				private set { SetValue(HasOutputTextProperty, value); }
			}

			////////////// BuildJob static methods ///////////////////////

			static BuildJob()
			{
				// Register the dependency properties
				JobStatusProperty = DependencyProperty.Register("JobStatus",
																typeof(BuildJobStatus), typeof(BuildJob),
																new FrameworkPropertyMetadata(BuildJobStatus.Pending,
																							  OnJobStatusChanged));

				JobStatusDisplayStringProperty = DependencyProperty.Register("JobStatusDisplayString",
																			 typeof(string), typeof(BuildJob),
																			 new FrameworkPropertyMetadata(InvalidJobStatusString));

				OutputTextProperty = DependencyProperty.Register("OutputText",
																 typeof(string), typeof(BuildJob),
																 new FrameworkPropertyMetadata(String.Empty, OnOutputTextChanged));

				HasOutputTextProperty = DependencyProperty.Register("HasOutputText",
																	typeof(bool), typeof(BuildJob),
																	new FrameworkPropertyMetadata(false));
			}

			private static void OnJobStatusChanged(DependencyObject Obj, DependencyPropertyChangedEventArgs Args)
			{
				BuildJob ThisBuildJob = (BuildJob)Obj;
				if (ThisBuildJob != null)
				{
					ThisBuildJob.JobStatusDisplayString = ThisBuildJob.GetJobStatusString();
				}
			}

			private static void OnOutputTextChanged(DependencyObject Obj, DependencyPropertyChangedEventArgs Args)
			{
				BuildJob ThisBuildJob = (BuildJob)Obj;
				if (ThisBuildJob != null)
				{
					ThisBuildJob.HasOutputText = !String.IsNullOrEmpty(ThisBuildJob.OutputText);
				}
			}

			////////////// BuildJob methods ///////////////////////

			public BuildJob(Utils.SafeProjectReference InProject, string InConfig, string InPlatform, BuildJobType InJobType)
			{
				_Project = InProject;
				_Config = InConfig;
				_Platform = InPlatform;
				_JobType = InJobType;

				_SolutionConfigString = Config + '|' + Platform;
				_DisplayString = GetDisplayString();
				JobStatusDisplayString = GetJobStatusString();
			}

			private string GetDisplayString()
			{
				if (Project == null || Config == null || Platform == null) return InvalidDisplayString;

				return String.Format("{0}: {1} [{2}]", Enum.GetName(typeof(BuildJobType), JobType), Project.Name,
									 SolutionConfigString);
			}

			private string GetJobStatusString()
			{
				return Enum.GetName(typeof(BuildJobStatus), JobStatus);
			}

			////////////// BuildJob fields ///////////////////////

			private const string InvalidDisplayString = "INVALIDBUILDJOB";
			private const string InvalidJobStatusString = "INVALIDSTATUS";
		}

		public class BuildJobSet
		{
			private readonly ObservableCollection<BuildJob> _BuildJobs = new ObservableCollection<BuildJob>();

			public string Name { get; set; }

			public ObservableCollection<BuildJob> BuildJobs
			{
				get { return _BuildJobs; }
			}

			public void DeepCopyJobsFrom(ObservableCollection<BuildJob> From)
			{
				foreach (var Job in From)
				{
					BuildJobs.Add(new BuildJob(Job.Project, Job.Config, Job.Platform, Job.JobType));
				}
			}

			public override string ToString()
			{
				return Name;
			}
		}

		public struct ProjectListItem
		{
			public Project Project { get; set; }

			public override string ToString()
			{
				ThreadHelper.ThrowIfNotOnUIThread();
				if (Project != null) return Project.Name;
				return "<None>";
			}
		}

		public class BatchBuilderToolState
		{
			public readonly ObservableCollection<BuildJobSet> _BuildJobSetsCollection = new ObservableCollection<BuildJobSet>();

			/// <summary>
			/// Called from the package class when there are options to be read out of the solution file.
			/// </summary>
			/// <param name="Stream">The stream to load the option data from.</param>
			public void LoadOptions(Stream Stream)
			{
				try
				{
					_BuildJobSetsCollection.Clear();

					using (BinaryReader Reader = new BinaryReader(Stream))
					{
						int SetCount = Reader.ReadInt32();

						for (int SetIdx = 0; SetIdx < SetCount; SetIdx++)
						{
							BuildJobSet LoadedSet = new BuildJobSet
							{
								Name = Reader.ReadString()
							};
							int JobCount = Reader.ReadInt32();
							for (int JobIdx = 0; JobIdx < JobCount; JobIdx++)
							{
								Utils.SafeProjectReference ProjectRef = new Utils.SafeProjectReference { FullName = Reader.ReadString(), Name = Reader.ReadString() };

								string Config = Reader.ReadString();
								string Platform = Reader.ReadString();

								if (Enum.TryParse(Reader.ReadString(), out BuildJob.BuildJobType JobType))
								{
									LoadedSet.BuildJobs.Add(new BuildJob(ProjectRef, Config, Platform, JobType));
								}
							}
							_BuildJobSetsCollection.Add(LoadedSet);
						}
					}

					if (_BuildJobSetsCollection.Count == 0)
					{
						BuildJobSet DefaultItem = new BuildJobSet { Name = "UntitledSet" };
						_BuildJobSetsCollection.Add(DefaultItem);
					}

					StateLoaded?.Invoke(this, null);
				}
				catch (Exception ex)
				{
					Exception AppEx = new ApplicationException("BatchBuilder failed to load options from .suo", ex);
					Logging.WriteLine(AppEx.ToString());
					throw AppEx;
				}
			}

			/// <summary>
			/// Called from the package class when there are options to be written to the solution file.
			/// </summary>
			/// <param name="Stream">The stream to save the option data to.</param>
			public void SaveOptions(Stream Stream)
			{
				try
				{
					using (BinaryWriter Writer = new BinaryWriter(Stream))
					{
						Writer.Write(_BuildJobSetsCollection.Count);
						foreach (var Set in _BuildJobSetsCollection)
						{
							Writer.Write(Set.Name);
							Writer.Write(Set.BuildJobs.Count);
							foreach (var Job in Set.BuildJobs)
							{
								Writer.Write(Job.Project.FullName);
								Writer.Write(Job.Project.Name);
								Writer.Write(Job.Config);
								Writer.Write(Job.Platform);
								Writer.Write(Enum.GetName(typeof(BuildJob.BuildJobType), Job.JobType) ?? "INVALIDJOBTYPE");
							}
						}
					}
				}
				catch (Exception ex)
				{
					Exception AppEx = new ApplicationException("BatchBuilder failed to save options to .suo", ex);
					Logging.WriteLine(AppEx.ToString());
					throw AppEx;
				}
			}

			// we are using a custom even instead of listening to the observable collection as we just want a even when the entire collection has been modified
			public event EventHandler StateLoaded;
		}

		////////////// Dependency properties ///////////////////////

		public static readonly DependencyProperty IsSolutionOpenProperty;
		public static readonly DependencyProperty IsSingleBuildJobSelectedProperty;
		public static readonly DependencyProperty BuildJobsProperty;
		public static readonly DependencyProperty IsDeletableSetSelectedProperty;
		public static readonly DependencyProperty BuildJobsPanelTitleProperty;
		public static readonly DependencyProperty OutputPanelTitleProperty;
		public static readonly DependencyProperty HasOutputProperty;
		public static readonly DependencyProperty IsBusyProperty;

		////////////// Dependency property wrappers ///////////////////////

		public bool IsSolutionOpen
		{
			get { return (bool)GetValue(IsSolutionOpenProperty); }
			set { SetValue(IsSolutionOpenProperty, value); }
		}

		public bool IsSingleBuildJobSelected
		{
			get { return (bool)GetValue(IsSingleBuildJobSelectedProperty); }
			set { SetValue(IsSingleBuildJobSelectedProperty, value); }
		}

		public ObservableCollection<BuildJob> BuildJobs
		{
			get { return (ObservableCollection<BuildJob>)GetValue(BuildJobsProperty); }
			set { SetValue(BuildJobsProperty, value); }
		}

		private readonly BuildJobSet _LastBuildJobsQueued = new BuildJobSet();
		public ObservableCollection<BuildJob> LastBuildJobsQueued
		{
			get { return _LastBuildJobsQueued.BuildJobs; }
		}

		public bool IsDeletableSetSelected
		{
			get { return (bool)GetValue(IsDeletableSetSelectedProperty); }
			set { SetValue(IsDeletableSetSelectedProperty, value); }
		}

		public string BuildJobsPanelTitle
		{
			get { return (string)GetValue(BuildJobsPanelTitleProperty); }
			set { SetValue(BuildJobsPanelTitleProperty, value); }
		}

		public string OutputPanelTitle
		{
			get { return (string)GetValue(OutputPanelTitleProperty); }
			set { SetValue(OutputPanelTitleProperty, value); }
		}

		public bool HasOutput
		{
			get { return (bool)GetValue(HasOutputProperty); }
			set { SetValue(HasOutputProperty, value); }
		}

		public bool IsBusy
		{
			get { return (bool)GetValue(IsBusyProperty); }
			set { SetValue(IsBusyProperty, value); }
		}

		////////////// 'Normal' Properties ///////////////////////

		public ObservableCollection<ProjectListItem> Projects
		{
			get { return _ProjectCollection; }
		}

		public ObservableCollection<string> Configs
		{
			get { return _ConfigCollection; }
		}

		public ObservableCollection<string> Platforms
		{
			get { return _PlatformCollection; }
		}

		public ObservableCollection<BuildJobSet> BuildJobSets
		{
			get { return _state._BuildJobSetsCollection; }
		}

		public string SelectedBuildJobSetName
		{
			set
			{
				if (String.IsNullOrEmpty(value)) return;

				BuildJobSet SelectedSet =
					_state._BuildJobSetsCollection.FirstOrDefault(
						Set => String.Compare(Set.Name, value, StringComparison.InvariantCultureIgnoreCase) == 0);

				if (SelectedSet == null)
				{
					SelectedSet = new BuildJobSet { Name = value };
					SelectedSet.DeepCopyJobsFrom(BuildJobs);
					_state._BuildJobSetsCollection.Add(SelectedSet);
				}

				SetCombo.SelectedItem = SelectedSet;
			}
		}

		////////////// static methods ///////////////////////

		/// <summary>
		/// Static consturctor initializes dependency properties
		/// </summary>
		static BatchBuilderToolControl()
		{
			// Register the dependency properties
			IsSolutionOpenProperty = DependencyProperty.Register("IsSolutionOpen",
																 typeof(bool), typeof(BatchBuilderToolControl),
																 new FrameworkPropertyMetadata(false));

			IsSingleBuildJobSelectedProperty = DependencyProperty.Register("IsSingleBuildJobSelected",
																		   typeof(bool), typeof(BatchBuilderToolControl),
																		   new FrameworkPropertyMetadata(false));

			BuildJobsProperty = DependencyProperty.Register("BuildJobs",
															typeof(ObservableCollection<BuildJob>),
															typeof(BatchBuilderToolControl),
															new FrameworkPropertyMetadata(new ObservableCollection<BuildJob>()));

			IsDeletableSetSelectedProperty = DependencyProperty.Register("IsDeletableSetSelected",
																		 typeof(bool), typeof(BatchBuilderToolControl),
																		 new FrameworkPropertyMetadata(false));

			BuildJobsPanelTitleProperty = DependencyProperty.Register("BuildJobsPanelTitle",
																	  typeof(string), typeof(BatchBuilderToolControl),
																	  new FrameworkPropertyMetadata(BuildJobsPanelPrefix));

			OutputPanelTitleProperty = DependencyProperty.Register("OutputPanelTitle",
																   typeof(string), typeof(BatchBuilderToolControl),
																   new FrameworkPropertyMetadata(OutputPanelTitlePrefix));

			HasOutputProperty = DependencyProperty.Register("HasOutput",
															typeof(bool), typeof(BatchBuilderToolControl),
															new FrameworkPropertyMetadata(false));

			IsBusyProperty = DependencyProperty.Register("IsBusy",
														 typeof(bool), typeof(BatchBuilderToolControl),
														 new FrameworkPropertyMetadata(false));
		}

		private static void DisplayBatchOutputText(string Text)
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			if (string.IsNullOrEmpty(Text)) return;

			UnrealVSPackage.Instance.DTE.ExecuteCommand("View.Output");

			var Pane = UnrealVSPackage.Instance.GetOutputPane(GuidList.BatchBuildPaneGuid, "UnrealVS - BatchBuild");
			if (Pane != null)
			{
				// Clear and activate the output pane.
				Pane.Clear();

				Pane.OutputStringThreadSafe(Text);

				// @todo: Activating doesn't seem to really bring the pane to front like we would expect it to.
				Pane.Activate();
			}
		}

		////////////// public methods ///////////////////////

		/// <summary>
		/// Default constructor
		/// </summary>
		public BatchBuilderToolControl(BatchBuilderToolState state)
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			_state = state;
			IsSolutionOpen = UnrealVSPackage.Instance.DTE.Solution.IsOpen;

			InitializeComponent();

			UnrealVSPackage.Instance.OnBuildBegin += OnBuildBegin;
			UnrealVSPackage.Instance.OnBuildDone += OnBuildDone;

			// Register for events that we care about
			UnrealVSPackage.Instance.StartupProjectSelector.StartupProjectListChanged += RefreshProjectCollection;
			UnrealVSPackage.Instance.OnSolutionOpened += delegate
															 {
																 IsSolutionOpen = true;
																 RefreshConfigAndPlatformCollections();
															 };
			UnrealVSPackage.Instance.OnSolutionClosed += delegate
															 {
																 IsSolutionOpen = false;
																 RefreshConfigAndPlatformCollections();
															 };

			RefreshProjectCollection(UnrealVSPackage.Instance.StartupProjectSelector.StartupProjectList);
			RefreshConfigAndPlatformCollections();

			BuildRadioButton.IsChecked = true;

			EnsureDefaultBuildJobSet();

			state.StateLoaded += BuildJobSetsCollectionOnCollectionChanged;
		}

		private void BuildJobSetsCollectionOnCollectionChanged(object sender, EventArgs eventArgs)
		{
			Logging.WriteLine("BuildJobSet Collection changed");

			if (SetCombo.SelectedItem == null)
			{
				SetCombo.SelectedItem = _state._BuildJobSetsCollection[0];
			}
		}

		/// <summary>
		/// Tick function called from the package via BatchBuilder.Tick()
		/// </summary>
		public void Tick()
		{
			ThreadHelper.ThrowIfNotOnUIThread();
			TickBuildQueue();
		}

		////////////// private methods ///////////////////////

		private void EnsureDefaultBuildJobSet()
		{
			if (_state._BuildJobSetsCollection.Count == 0)
			{
				BuildJobSet DefaultItem = new BuildJobSet { Name = "UntitledSet" };
				_state._BuildJobSetsCollection.Add(DefaultItem);
				if (SetCombo != null)
				{
					SetCombo.SelectedItem = DefaultItem;
				}
			}
		}

		private void RefreshProjectCollection(object sender, StartupProjectSelector.StartupProjectListChangedEventArgs e)
		{
			RefreshProjectCollection(e.StartupProjects);
		}

		private void RefreshProjectCollection(Project[] StartupProjects)
		{
			_ProjectCollection.Clear();

			foreach (var Project in StartupProjects)
			{
				_ProjectCollection.Add(new ProjectListItem { Project = Project });
			}
		}

		private void RefreshConfigAndPlatformCollections()
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			_ConfigCollection.Clear();
			_PlatformCollection.Clear();

			if (!IsSolutionOpen)
			{
				_state._BuildJobSetsCollection.Clear();
			}

			Utils.GetSolutionConfigsAndPlatforms(out string[] RefreshConfigs, out string[] RefreshPlatforms);

			foreach (string Config in RefreshConfigs)
			{
				_ConfigCollection.Add(Config);
			}
			foreach (string Platform in RefreshPlatforms)
			{
				_PlatformCollection.Add(Platform);
			}

			EnsureDefaultBuildJobSet();
		}

		private void AddButtonClick(object Sender, RoutedEventArgs E)
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			BuildJob[] Buildables = GetSelectedBuildableOnLeft();

			if (Buildables.Length > 0)
			{
				foreach (var BuildJob in Buildables)
				{
					BuildJobs.Add(BuildJob);
				}
				JobsListTab.IsSelected = true;
			}
		}

		private void RemoveButtonClick(object Sender, RoutedEventArgs E)
		{
			BuildJob[] Buildables = GetSelectedBuildableOnRight();

			foreach (var BuildJob in Buildables)
			{
				BuildJobs.Remove(BuildJob);
			}

			JobsListTab.IsSelected = true;
		}

		private void UpButtonClick(object Sender, RoutedEventArgs E)
		{
			if (!IsSingleBuildJobSelected) return;

			BuildJob SelectedJob = (BuildJob)BuildJobsList.SelectedItem;

			int CurrentIdx = BuildJobs.IndexOf(SelectedJob);
			if (CurrentIdx > 0)
			{
				BuildJobs.Move(CurrentIdx, CurrentIdx - 1);
			}
		}

		private void DownButtonClick(object Sender, RoutedEventArgs E)
		{
			if (!IsSingleBuildJobSelected) return;

			BuildJob SelectedJob = (BuildJob)BuildJobsList.SelectedItem;

			int CurrentIdx = BuildJobs.IndexOf(SelectedJob);
			if (CurrentIdx < BuildJobs.Count - 1)
			{
				BuildJobs.Move(CurrentIdx, CurrentIdx + 1);
			}
		}

		private BuildJob[] GetSelectedBuildableOnLeft()
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			BuildJob.BuildJobType JobType = GetSelectedBuildJobType();

			List<BuildJob> Buildables = new List<BuildJob>();

			foreach (ProjectListItem Project in ProjectsList.SelectedItems)
			{
				foreach (string Config in ConfigsList.SelectedItems)
				{
					foreach (string Platform in PlatformsList.SelectedItems)
					{
						Buildables.Add(
							new BuildJob(new Utils.SafeProjectReference { FullName = Project.Project.FullName, Name = Project.Project.Name },
										 Config, Platform, JobType));
					}
				}
			}

			return Buildables.ToArray();
		}

		private BuildJob[] GetSelectedBuildableOnRight()
		{
			List<BuildJob> Buildables = new List<BuildJob>();

			foreach (BuildJob BuildJob in BuildJobsList.SelectedItems)
			{
				Buildables.Add(BuildJob);
			}

			return Buildables.ToArray();
		}

		private BuildJob.BuildJobType GetSelectedBuildJobType()
		{
			if (BuildRadioButton.IsChecked.HasValue && BuildRadioButton.IsChecked.Value)
			{
				return BuildJob.BuildJobType.Build;
			}
			if (RebuildRadioButton.IsChecked.HasValue && RebuildRadioButton.IsChecked.Value)
			{
				return BuildJob.BuildJobType.Rebuild;
			}
			return BuildJob.BuildJobType.Clean;
		}

		private void OnBuildBegin(out int Cancel)
		{
			Cancel = 0;
			if (IsBusy && _ActiveBuildJob == null)
			{
				Cancel = 1;
			}
		}

		private void OnBuildDone(bool bSucceeded, bool bModified, bool bWasCancelled)
		{
			ThreadHelper.ThrowIfNotOnUIThread();
			if (_ActiveBuildJob != null)
			{
				if (bWasCancelled)
				{
					_ActiveBuildJob.JobStatus = BuildJob.BuildJobStatus.Cancelled;
					if (IsBusy)
					{
						foreach (var Job in _BuildQueue)
						{
							Job.JobStatus = BuildJob.BuildJobStatus.Cancelled;
							Job.OutputText = GetBuildJobOutputText(Job, null);
						}
						_BuildQueue.Clear();
					}
				}
				else if (bSucceeded)
				{
					_ActiveBuildJob.JobStatus = BuildJob.BuildJobStatus.Complete;
				}
				else
				{
					_ActiveBuildJob.JobStatus = BuildJob.BuildJobStatus.Failed;
				}

				_ActiveBuildJob.OutputText = GetBuildJobOutputText(_ActiveBuildJob, _BuildJobStartTime);
			}

			_ActiveBuildJob = null;
			UpdateBusyState();
		}

		private void OnBuildJobsSelectionChanged(object sender, SelectionChangedEventArgs e)
		{
			IsSingleBuildJobSelected = BuildJobsList != null && BuildJobsList.SelectedItems.Count == 1;
		}

		private void OnSetComboSelectionChanged(object sender, SelectionChangedEventArgs e)
		{
			if (SetCombo.SelectedItem != null)
			{
				_LastSelectedBuildJobSet = (BuildJobSet)SetCombo.SelectedItem;
				BuildJobs = _LastSelectedBuildJobSet.BuildJobs;
				IsDeletableSetSelected = _state._BuildJobSetsCollection.Count > 1;
				BuildJobsPanelTitle = String.Format("{0} ({1})", BuildJobsPanelPrefix, _LastSelectedBuildJobSet.Name);
			}
			if (!_state._BuildJobSetsCollection.Contains(_LastSelectedBuildJobSet))
			{
				_LastSelectedBuildJobSet = null;
				BuildJobs = new ObservableCollection<BuildJob>();
				IsDeletableSetSelected = false;
				BuildJobsPanelTitle = BuildJobsPanelPrefix;
			}
		}

		private void OnSetComboKeyDown(object sender, KeyEventArgs e)
		{
			if (e.Key == Key.Enter)
			{
				SelectedBuildJobSetName = SetCombo.Text;
			}
		}

		private void OnDeleteButtonClick(object sender, RoutedEventArgs e)
		{
			if (_LastSelectedBuildJobSet != null)
			{
				int SelectionIdx = _state._BuildJobSetsCollection.IndexOf(_LastSelectedBuildJobSet);
				if (SelectionIdx >= 0 && SelectionIdx < _state._BuildJobSetsCollection.Count)
				{
					SetCombo.SelectedItem = null;
					_state._BuildJobSetsCollection.RemoveAt(SelectionIdx);
					if (!(SelectionIdx < _state._BuildJobSetsCollection.Count))
					{
						SelectionIdx--;
					}
					if (SelectionIdx >= 0)
					{
						SetCombo.SelectedIndex = SelectionIdx;
					}
				}
			}
		}

		private void OnStartStopButtonClick(object sender, RoutedEventArgs e)
		{
			ThreadHelper.ThrowIfNotOnUIThread();
			if (!IsBusy)
			{
				// Start
				RunBuildJobs();
			}
			else
			{
				// Stop
				UnrealVSPackage.Instance.DTE.ExecuteCommand("Build.Cancel");
				if (_ActiveBuildJob != null)
				{
					_ActiveBuildJob.JobStatus = BuildJob.BuildJobStatus.Cancelled;
					_ActiveBuildJob.OutputText = GetBuildJobOutputText(_ActiveBuildJob, _BuildJobStartTime);
					_ActiveBuildJob = null;
				}
				foreach (var BuildJob in _BuildQueue)
				{
					BuildJob.JobStatus = BuildJob.BuildJobStatus.Cancelled;
					BuildJob.OutputText = GetBuildJobOutputText(BuildJob, null);
				}
				_BuildQueue.Clear();
				UpdateBusyState();
			}
		}

		private void RunBuildJobs()
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			if (BuildJobs == null)
				throw new ArgumentNullException(nameof(BuildJobs));

			if (_LastBuildJobsQueued == null)
				throw new ArgumentNullException(nameof(_LastBuildJobsQueued));

			int BuildJobCount = BuildJobs.Count;
			SanityCheckBuildJobs();

			if (BuildJobCount > BuildJobs.Count)
			{
				if (VSConstants.S_OK != VsShellUtilities.ShowMessageBox(ServiceProvider.GlobalProvider,
																		"WARNING: Some build jobs were deleted because they were invalid.",
																		"UnrealVS",
																		OLEMSGICON.OLEMSGICON_WARNING,
																		OLEMSGBUTTON.OLEMSGBUTTON_OKCANCEL,
																		OLEMSGDEFBUTTON.OLEMSGDEFBUTTON_FIRST))
				{
					return;
				}
			}

			_BuildQueue.Clear();
			_LastBuildJobsQueued.BuildJobs.Clear();
			_LastBuildJobsQueued.DeepCopyJobsFrom(BuildJobs);
			foreach (var Job in _LastBuildJobsQueued.BuildJobs)
			{
				_BuildQueue.Enqueue(Job);
			}

			HasOutput = _BuildQueue.Count > 0;
			if (_LastSelectedBuildJobSet != null)
			{
				OutputPanelTitle = String.Format("{0} ({1})", OutputPanelTitlePrefix, _LastSelectedBuildJobSet.Name);
			}
			OutputTab.IsSelected = HasOutput;

			UpdateBusyState();
		}

		private void UpdateBusyState()
		{
			IsBusy = _BuildQueue.Count > 0 || _ActiveBuildJob != null;
		}

		private void SanityCheckBuildJobs(Project[] CheckProjects)
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			// check that project, config and platform strings in build jobs are valid
			// remove invalid ones
			Utils.GetSolutionConfigsAndPlatforms(out string[] SolutionConfigs, out string[] SolutionPlatforms);

			foreach (var JobSet in _state._BuildJobSetsCollection)
			{
				for (int JobIdx = JobSet.BuildJobs.Count - 1; JobIdx >= 0; JobIdx--)
				{
					BuildJob Job = JobSet.BuildJobs[JobIdx];

#pragma warning disable VSTHRD010 // Invoke single-threaded types on Main thread
					bool bProjectOkay = CheckProjects.Any(Proj => String.CompareOrdinal(Job.Project.FullName, Proj.FullName) == 0);
#pragma warning restore VSTHRD010 // Invoke single-threaded types on Main thread
					bool bConfigOkay = SolutionConfigs.Any(Config => String.CompareOrdinal(Job.Config, Config) == 0);
					bool bPlatformOkay = SolutionPlatforms.Any(Platform => String.CompareOrdinal(Job.Platform, Platform) == 0);

					if (!bProjectOkay || !bConfigOkay || !bPlatformOkay)
					{
						JobSet.BuildJobs.RemoveAt(JobIdx);
					}
				}
			}
		}

		private void SanityCheckBuildJobs()
		{
			ThreadHelper.ThrowIfNotOnUIThread();
			SanityCheckBuildJobs(Utils.GetAllProjectsFromDTE());
		}

		private void TickBuildQueue()
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			if (_ActiveBuildJob != null || _BuildQueue.Count == 0) return;

			while (_ActiveBuildJob == null && _BuildQueue.Count > 0)
			{
				BuildJob Job = _BuildQueue.Dequeue();

				Project Project = Job.Project.GetProjectSlow();

				if (Project != null)
				{
					Utils.ExecuteProjectBuild(
						Project,
						Job.Config,
						Job.Platform,
						Job.JobType,
						delegate
						{
							// Job starting
							_ActiveBuildJob = Job;
							_ActiveBuildJob.JobStatus = BuildJob.BuildJobStatus.Executing;
							_BuildJobStartTime = DateTime.Now;
						},
						delegate
						{
							// Job failed to start - clear active job
							_ActiveBuildJob.JobStatus = BuildJob.BuildJobStatus.FailedToStart;
							_ActiveBuildJob.OutputText = GetBuildJobOutputText(_ActiveBuildJob, _BuildJobStartTime);
							_ActiveBuildJob = null;
						});
				}
			}
		}

		private string GetBuildJobOutputText(BuildJob Job, DateTime? StartTime)
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			var OutputText = new StringBuilder();

			var Window = UnrealVSPackage.Instance.DTE.Windows.Item(Constants.vsWindowKindOutput);
			if (Window != null)
			{
				var OutputWindow = Window.Object as OutputWindow;
				if (OutputWindow != null)
				{
					var OutputWindowPane = OutputWindow.OutputWindowPanes.Item("Build");
					if (OutputWindowPane != null)
					{
						var TextStartPoint = (EditPoint2)OutputWindowPane.TextDocument.StartPoint.CreateEditPoint();
						string BuildOutputText = TextStartPoint.GetText(OutputWindowPane.TextDocument.EndPoint);

						OutputText.AppendLine("========== Build Output ==========");
						OutputText.AppendLine(string.Empty);
						OutputText.Append(BuildOutputText);
						OutputText.AppendLine(string.Empty);
					}
				}
			}

			int JobIndex = LastBuildJobsQueued.IndexOf(Job) + 1;
			int JobCount = LastBuildJobsQueued.Count;

			OutputText.AppendLine(string.Format("========== Batch Builder {0} - Job {1} of {2} ==========", OutputPanelTitle,
												JobIndex, JobCount));
			OutputText.AppendLine(string.Empty);
			OutputText.AppendLine("------ " + Job.DisplayString + " ------");
			OutputText.AppendLine("Status: " + Job.JobStatusDisplayString);

			if (StartTime.HasValue)
			{
				OutputText.AppendLine("Started: " + StartTime.Value.ToLongTimeString());

				if (Job.JobStatus == BuildJob.BuildJobStatus.Complete ||
					Job.JobStatus == BuildJob.BuildJobStatus.Failed ||
					Job.JobStatus == BuildJob.BuildJobStatus.Cancelled)
				{
					DateTime EndTime = DateTime.Now;
					OutputText.AppendLine("Ended: " + EndTime.ToLongTimeString());
					OutputText.AppendLine(string.Format("Execution time: {0:F2} seconds", (EndTime - StartTime.Value).TotalSeconds));
				}
			}

			return OutputText.ToString();
		}

		private void OpenItemOutputCommandExecuted(object sender, ExecutedRoutedEventArgs e)
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			if (!(e.Parameter is BuildJob Job)) return;

			DisplayBatchOutputText(Job.OutputText);
		}

		private void OnDblClickBuildListItem(object sender, MouseButtonEventArgs e)
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			if (!(sender is FrameworkElement Elem)) return;

			if (!(Elem.DataContext is BuildJob Job)) return;

			// Switch the Startup Project and the build config and platform to match this Job
			if (Job.Project != null)
			{
				IsBusy = true;

				var Project = Job.Project.GetProjectSlow();

				if (Project != null)
				{
					// Switch to this project
					var ProjectHierarchy = Utils.ProjectToHierarchyObject(Project);
					UnrealVSPackage.Instance.SolutionBuildManager.set_StartupProject(ProjectHierarchy);

					// Switch the build config
					Utils.SetActiveSolutionConfiguration(Job.Config, Job.Platform);
				}

				IsBusy = false;
			}
		}

		private void OnDblClickBuildingListItem(object sender, MouseButtonEventArgs e)
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			if (!(sender is FrameworkElement Elem)) return;

			if (!(Elem.DataContext is BuildJob Job)) return;

			DisplayBatchOutputText(Job.OutputText);
		}

		////////////// Fields ///////////////////////

		private const string BuildJobsPanelPrefix = "Build Jobs";
		private const string OutputPanelTitlePrefix = "Output";

		private readonly Queue<BuildJob> _BuildQueue = new Queue<BuildJob>();
		private BuildJob _ActiveBuildJob;
		private DateTime _BuildJobStartTime;
		private BuildJobSet _LastSelectedBuildJobSet;

		private readonly ObservableCollection<ProjectListItem> _ProjectCollection = new ObservableCollection<ProjectListItem>();
		private readonly ObservableCollection<string> _ConfigCollection = new ObservableCollection<string>();
		private readonly ObservableCollection<string> _PlatformCollection = new ObservableCollection<string>();
	}
}