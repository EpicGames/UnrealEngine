// Copyright Epic Games, Inc. All Rights Reserved.

using Microsoft.VisualStudio.Shell;
using System;
using System.Runtime.InteropServices;

namespace UnrealVS
{
	[Guid(GuidList.UnrealVSBatchBuildToolWindowPersistanceString)]
	internal class BatchBuilderToolWindow : ToolWindowPane
	{
		/// <summary>
		/// Standard constructor for the tool window.
		/// </summary>
		public BatchBuilderToolWindow() :
			base(null)
		{
			// Set the window title reading it from the resources.
			Caption = Resources.BatchBuilderWindowTitle;
			// Set the image that will appear on the tab of the window frame
			// when docked with an other window
			// The resource ID correspond to the one defined in the resx file
			// while the Index is the offset in the bitmap strip. Each image in
			// the strip being 16x16.
			BitmapResourceID = 200;
			BitmapIndex = 0;

			// This is the user control hosted by the tool window; Note that, even if this class implements IDisposable,
			// we are not calling Dispose on this object. This is because ToolWindowPane calls Dispose on 
			// the object returned by the Content property.
			base.Content = BatchBuilder.ToolControl;
		}
	}
}
