// Copyright Epic Games, Inc. All Rights Reserved.

using EnvDTE;
using Microsoft.VisualStudio.Shell;
using System;
using System.Collections.Generic;
using System.ComponentModel.Design;

namespace UnrealVS
{
	class BuildAndProfile : IDisposable
	{
		private static readonly int ProjectRebuildAndProfileButtonID = 0x1071;

		private static readonly int FolderUnrealVSBuildButtonID = 0x1170;
		private static readonly int FolderUnrealVSBuildNonUnityButtonID = 0x1171;
		private static readonly int FolderUnrealVSRebuildAndProfileButtonID = 0x1172;
		private static readonly int FolderUnrealVSRebuildAndProfileNonUnityButtonID = 0x1173;

		private static bool IsProjectCommandID(int CommandID)
		{
			return CommandID == ProjectRebuildAndProfileButtonID;
		}

		private static bool IsProfileCommandID(int CommandID)
		{
			return CommandID == ProjectRebuildAndProfileButtonID
				|| CommandID == FolderUnrealVSRebuildAndProfileButtonID
				|| CommandID == FolderUnrealVSRebuildAndProfileNonUnityButtonID;
		}

		private static bool IsModuleBuildCommandID(int CommandID)
		{
			return CommandID == FolderUnrealVSBuildButtonID
				|| CommandID == FolderUnrealVSBuildNonUnityButtonID
				|| CommandID == FolderUnrealVSRebuildAndProfileButtonID
				|| CommandID == FolderUnrealVSRebuildAndProfileNonUnityButtonID;
		}

		private static bool IsNonUnityBuildCommandID(int CommandID)
		{
			return CommandID == FolderUnrealVSBuildNonUnityButtonID
				|| CommandID == FolderUnrealVSRebuildAndProfileNonUnityButtonID;
		}

		private UnrealProjectBuilder ProjectBuilder = new UnrealProjectBuilder();

		public BuildAndProfile()
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			// Project actions
			AddCommandInternal(ProjectRebuildAndProfileButtonID);

			// Folder actions
			AddCommandInternal(FolderUnrealVSBuildButtonID);
			AddCommandInternal(FolderUnrealVSBuildNonUnityButtonID);
			AddCommandInternal(FolderUnrealVSRebuildAndProfileButtonID);
			AddCommandInternal(FolderUnrealVSRebuildAndProfileNonUnityButtonID);

			void AddCommandInternal(int CommandID)
			{
				var MenuCommand = new OleMenuCommand(BuildCallback, new CommandID(GuidList.UnrealVSCmdSet, CommandID));
				MenuCommand.BeforeQueryStatus += BuildQueryStatus;
				UnrealVSPackage.Instance.MenuCommandService.AddCommand(MenuCommand);
			}
		}

		public void Dispose()
		{
			ProjectBuilder.Dispose();
		}

		private void BuildQueryStatus(object Sender, EventArgs Args)
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			if (Sender is MenuCommand BuildCommand)
			{
				bool bEnabled = true;

				if (IsProfileCommandID(BuildCommand.CommandID.ID) && !CompileScoreHelper.IsInstalled())
				{
					bEnabled = false;
				}
				else if (IsModuleBuildCommandID(BuildCommand.CommandID.ID) && !Utils.GetSelectedModule(out _, out _))
				{
					bEnabled = false;
				}

				BuildCommand.Enabled = bEnabled;
			}
		}

		private void BuildCallback(object Sender, EventArgs Args)
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			if (Sender is MenuCommand BuildCommand)
			{
				List<string> ExtraArguments = new List<string>();

				int CommandID = BuildCommand.CommandID.ID;

				if (IsNonUnityBuildCommandID(CommandID))
				{
					ExtraArguments.Add("-DisableUnity");
					// use -NoLink to prevent compiling all dependencies NonUnity as well.
					ExtraArguments.Add("-NoLink");
				}

				if (IsProfileCommandID(CommandID))
				{
					ExtraArguments.AddRange(new List<string>
					{
						// When profiling:
						//   Force disable UBA to force a local build.
						"-NoUba",
						"-NoVfs",
						//   Force use of MSVC since this is specifically for CompileScore.
						"-Compiler=VisualStudio",
						//   Force a unique build environment, might also add more compile options.
						"-ProfilingCompile",
						//   Force a rebuild; by requesting a profile, you are asking for a full profile, not an incremental one.
						"-Rebuild"
					});

					if (IsProjectCommandID(CommandID))
					{
						// Get the project clicked in the solution explorer by accessing the current selection and converting to a Project if possible.
						if (Utils.GetCurrentlySelectedProject() is Project SelectedProject)
						{
							ProjectBuilder.BuildProject(SelectedProject, null, ExtraArguments, bProfiling: true);
						}
					}
					else if (IsModuleBuildCommandID(CommandID))
					{
						// use -NoLink to prevent profiling all dependencies as well.
						ExtraArguments.Add("-NoLink");

						if (Utils.GetSelectedModule(out Project ModuleProject, out string ModuleName))
						{
							ProjectBuilder.BuildModule(ModuleProject, ModuleName, ExtraArguments, bProfiling: true);
						}
					}
				}
				else if (IsModuleBuildCommandID(CommandID))
				{
					if (Utils.GetSelectedModule(out Project ModuleProject, out string ModuleName))
					{
						ProjectBuilder.BuildModule(ModuleProject, ModuleName, ExtraArguments, bProfiling: false);
					}
				}
			}
		}
	}
}
