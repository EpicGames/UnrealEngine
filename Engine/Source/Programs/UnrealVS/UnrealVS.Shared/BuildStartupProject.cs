// Copyright Epic Games, Inc. All Rights Reserved.

using EnvDTE80;
using Microsoft.VisualStudio.Shell;
using Microsoft.VisualStudio.Shell.Interop;
using System;
using System.ComponentModel.Design;

namespace UnrealVS
{
	class BuildStartupProject
	{
		const int BuildStartupProjectButtonID = 0x1070;

		public BuildStartupProject()
		{
			// BuildStartupProjectButton
			{
				var CommandID = new CommandID(GuidList.UnrealVSCmdSet, BuildStartupProjectButtonID);
				var BuildStartupProjectButtonCommand = new MenuCommand(new EventHandler(BuildStartupProjectButtonHandler), CommandID);
				UnrealVSPackage.Instance.MenuCommandService.AddCommand(BuildStartupProjectButtonCommand);
			}

		}


		/// Called when 'BuildStartupProject' button is clicked
		void BuildStartupProjectButtonHandler(object Sender, EventArgs Args)
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			// Grab the current startup project
			UnrealVSPackage.Instance.SolutionBuildManager.get_StartupProject(out IVsHierarchy ProjectHierarchy);
			if (ProjectHierarchy != null)
			{
				var StartupProject = Utils.HierarchyObjectToProject(ProjectHierarchy);

				if (StartupProject != null)
				{
					// Get the active solution configuration
					var ActiveConfiguration =
						(SolutionConfiguration2)UnrealVSPackage.Instance.DTE.Solution.SolutionBuild.ActiveConfiguration;
					var SolutionConfiguration = ActiveConfiguration.Name;
					var SolutionPlatform = ActiveConfiguration.PlatformName;

					// Combine the active solution configuration and platform into a string that Visual Studio
					// can use to build the startup project (e.g. "Release|x64")
					var BuildPlatformAndConfiguration = SolutionConfiguration + "|" + SolutionPlatform;

					// Make sure the Output window is visible
					UnrealVSPackage.Instance.DTE.ExecuteCommand("View.Output");

					// Kick off the build!
					UnrealVSPackage.Instance.DTE.Solution.SolutionBuild.BuildProject(
						BuildPlatformAndConfiguration,
						StartupProject.UniqueName,
						WaitForBuildToFinish: false);
				}
			}
		}
	}
}
