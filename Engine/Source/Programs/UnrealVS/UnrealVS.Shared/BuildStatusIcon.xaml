<!--Copyright Epic Games, Inc. All Rights Reserved.-->

<UserControl x:Class="UnrealVS.BuildStatusIcon"
			 xmlns="http://schemas.microsoft.com/winfx/2006/xaml/presentation"
			 xmlns:x="http://schemas.microsoft.com/winfx/2006/xaml"
			 xmlns:mc="http://schemas.openxmlformats.org/markup-compatibility/2006" 
			 xmlns:d="http://schemas.microsoft.com/expression/blend/2008" 
			 mc:Ignorable="d" 
			 d:DesignHeight="200" d:DesignWidth="200"
			 Foreground="Black">
	<UserControl.Resources>
		<LinearGradientBrush x:Key="InnerBrush" EndPoint="0.5,1" StartPoint="0.5,0">
			<GradientStop Color="{Binding RelativeSource={RelativeSource FindAncestor, AncestorType={x:Type UserControl}}, Path=ColorLo}" Offset="1"/>
			<GradientStop Color="{Binding RelativeSource={RelativeSource FindAncestor, AncestorType={x:Type UserControl}}, Path=ColorHi}" Offset="0"/>
		</LinearGradientBrush>
	</UserControl.Resources>
	<Grid>
		<Ellipse Fill="Black" />
		<Ellipse Fill="{DynamicResource InnerBrush}" RenderTransformOrigin="0.5,0.5">
			<Ellipse.RenderTransform>
				<TransformGroup>
					<ScaleTransform ScaleX="0.85" ScaleY="0.85"/>
					<SkewTransform/>
					<RotateTransform/>
					<TranslateTransform/>
				</TransformGroup>
			</Ellipse.RenderTransform>
		</Ellipse>
	</Grid>
</UserControl>
