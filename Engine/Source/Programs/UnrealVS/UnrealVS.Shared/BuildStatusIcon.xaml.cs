// Copyright Epic Games, Inc. All Rights Reserved.

using System.Windows;
using System.Windows.Controls;
using System.Windows.Media;

namespace UnrealVS
{
	/// <summary>
	/// Icon control used on the list of building jobs to show the job status
	/// </summary>
	public partial class BuildStatusIcon : UserControl
	{
		// Dependency properties
		public static readonly DependencyProperty BuildStatusProperty;
		private static readonly DependencyProperty ColorLoProperty;
		private static readonly DependencyProperty ColorHiProperty;

		public BatchBuilderToolControl.BuildJob.BuildJobStatus BuildStatus
		{
			get { return (BatchBuilderToolControl.BuildJob.BuildJobStatus)GetValue(BuildStatusProperty); }
			set { SetValue(BuildStatusProperty, value); }
		}
		private Color ColorLo
		{
			get { return (Color)GetValue(ColorLoProperty); }
			set { SetValue(ColorLoProperty, value); }
		}
		private Color ColorHi
		{
			get { return (Color)GetValue(ColorHiProperty); }
			set { SetValue(ColorHiProperty, value); }
		}

		static BuildStatusIcon()
		{
			BuildStatusProperty = DependencyProperty.Register("BuildStatus",
															  typeof(BatchBuilderToolControl.BuildJob.BuildJobStatus),
															  typeof(BuildStatusIcon),
															  new FrameworkPropertyMetadata(
																  BatchBuilderToolControl.BuildJob.BuildJobStatus.Pending,
																  OnBuildStatusChanged));
			ColorLoProperty = DependencyProperty.Register("ColorLo",
															  typeof(Color),
															  typeof(BuildStatusIcon),
															  new FrameworkPropertyMetadata(
																  GetColorLo(BatchBuilderToolControl.BuildJob.BuildJobStatus.Pending)));
			ColorHiProperty = DependencyProperty.Register("ColorHi",
															  typeof(Color),
															  typeof(BuildStatusIcon),
															  new FrameworkPropertyMetadata(
																  GetColorHi(BatchBuilderToolControl.BuildJob.BuildJobStatus.Pending)));
		}

		private static void OnBuildStatusChanged(DependencyObject Obj, DependencyPropertyChangedEventArgs Args)
		{
			var ThisObj = (BuildStatusIcon)Obj;
			if (ThisObj != null)
			{
				ThisObj.ColorLo = GetColorLo((BatchBuilderToolControl.BuildJob.BuildJobStatus)Args.NewValue);
				ThisObj.ColorHi = GetColorHi((BatchBuilderToolControl.BuildJob.BuildJobStatus)Args.NewValue);
			}
		}

		public BuildStatusIcon()
		{
			InitializeComponent();
		}

		private static Color GetColorLo(BatchBuilderToolControl.BuildJob.BuildJobStatus Status)
		{
			switch (Status)
			{
				case BatchBuilderToolControl.BuildJob.BuildJobStatus.Pending:
					return Colors.LightGray;
				case BatchBuilderToolControl.BuildJob.BuildJobStatus.Executing:
					return Colors.SlateGray;
				case BatchBuilderToolControl.BuildJob.BuildJobStatus.Failed:
					return Colors.Red;
				case BatchBuilderToolControl.BuildJob.BuildJobStatus.FailedToStart:
					return Colors.Red;
				case BatchBuilderToolControl.BuildJob.BuildJobStatus.Cancelled:
					return Colors.DarkOrange;
				case BatchBuilderToolControl.BuildJob.BuildJobStatus.Complete:
					return Colors.Green;
				default:
					return Colors.Blue;
			}
		}

		private static Color GetColorHi(BatchBuilderToolControl.BuildJob.BuildJobStatus Status)
		{
			switch (Status)
			{
				case BatchBuilderToolControl.BuildJob.BuildJobStatus.Pending:
					return Colors.DarkGray;
				case BatchBuilderToolControl.BuildJob.BuildJobStatus.Executing:
					return Colors.DarkGray;
				case BatchBuilderToolControl.BuildJob.BuildJobStatus.Failed:
					return Colors.LightCoral;
				case BatchBuilderToolControl.BuildJob.BuildJobStatus.FailedToStart:
					return Colors.LightCoral;
				case BatchBuilderToolControl.BuildJob.BuildJobStatus.Cancelled:
					return Colors.LightSalmon;
				case BatchBuilderToolControl.BuildJob.BuildJobStatus.Complete:
					return Colors.LightGreen;
				default:
					return Colors.LightBlue;
			}
		}
	}
}
