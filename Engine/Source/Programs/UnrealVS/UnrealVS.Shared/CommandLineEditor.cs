// Copyright Epic Games, Inc. All Rights Reserved.

using EnvDTE;
using EnvDTE80;
using Microsoft.VisualStudio;
using Microsoft.VisualStudio.Shell;
using Microsoft.VisualStudio.Shell.Interop;
using Microsoft.VisualStudio.Threading;
using System;
using System.Collections.Generic;
using System.ComponentModel.Design;
using System.IO;
using System.Linq;
using System.Runtime.InteropServices;
using System.Text.Json;
using System.Text.Json.Serialization;
using System.Threading.Tasks;
using System.Timers;

namespace UnrealVS
{
	internal class CommandLineEditor : IDisposable
	{
		private class LaunchSettingsJson
		{
			[JsonPropertyName("profiles")]
			public Dictionary<string, LaunchSettingsProfile> Profiles { get; set; }

			public class LaunchSettingsProfile
			{
				[JsonPropertyName("commandName")]
				public string CommandName { get; set; }

				[JsonPropertyName("commandLineArgs")]
				public string CommandLineArgs { get; set; }

				[JsonExtensionData]
				public Dictionary<string, JsonElement> ExtensionData { get; set; }
			}

			[JsonExtensionData]
			public Dictionary<string, JsonElement> ExtensionData { get; set; }
		}
		/** constants */

		private const int ComboID = 0x1030;
		private const int ComboListID = 0x1040;
		private const int ComboListCountMax = 32;

		/** methods */

		public CommandLineEditor()
		{
		}


		public void Initialize()
		{
			// Create the handlers for our commands
			{
				// CommandLineCombo
				var ComboCommandID = new CommandID(GuidList.UnrealVSCmdSet, ComboID);
				ComboCommand = new OleMenuCommand(new EventHandler(ComboHandler), ComboCommandID);
				UnrealVSPackage.Instance.MenuCommandService.AddCommand(ComboCommand);

				// CommandLineComboList
				var ComboListCommandID = new CommandID(GuidList.UnrealVSCmdSet, ComboListID);
				ComboCommandList = new OleMenuCommand(new EventHandler(ComboListHandler), ComboListCommandID);
				UnrealVSPackage.Instance.MenuCommandService.AddCommand(ComboCommandList);
			}

			// Register for events that we care about
			UnrealVSPackage.Instance.OnSolutionOpened += UpdateCommandLineCombo;
			UnrealVSPackage.Instance.OnSolutionClosed += UpdateCommandLineCombo;
			UnrealVSPackage.Instance.OnStartupProjectChanged += (p) =>
			{
				ThreadHelper.ThrowIfNotOnUIThread();
				UpdateCommandLineCombo();
			};
			UnrealVSPackage.Instance.OnStartupProjectPropertyChanged += OnStartupProjectPropertyChanged;
			UnrealVSPackage.Instance.OnStartupProjectConfigChanged += OnStartupProjectConfigChanged;

			UpdateCommandLineCombo();
		}

		/// <summary>
		/// Called when a startup project property has changed.  We'll refresh our interface.
		/// </summary>
		public void OnStartupProjectPropertyChanged(UInt32 itemid, Int32 propid, UInt32 flags)
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			// Filter out Helix VS plugin sending thousands of VSHPROPID_StateIconIndex in the solution
			// This event type is not usefull at all for the commandline editor
			if (propid == (Int32)__VSHPROPID.VSHPROPID_StateIconIndex)
			{
				return;
			}

			UnrealVSPackage.Instance.SolutionBuildManager.get_StartupProject(out IVsHierarchy ProjectHierarchy);
			if (ProjectHierarchy != null)
			{
				// @todo: filter this so that we only respond to changes in the command line property
				// Setup a timer to prevent any more performance problem from spamming events
				if (UpdateCommandLineComboTimer == null)
				{
					UpdateCommandLineComboTimer = new System.Timers.Timer(1000)
					{
						AutoReset = false,
					};
					UpdateCommandLineComboTimer.Elapsed += OnUpdateCommandLineCombo;
				}
				// Restart timer to raise the event only after 1s of no notification
				UpdateCommandLineComboTimer.Stop();
				UpdateCommandLineComboTimer.Start();
			}
		}

		/// <summary>
		/// Timer callback to UpdateCommandLineCombo
		/// </summary>
		private void OnUpdateCommandLineCombo(Object _Source, ElapsedEventArgs _Args)
		{
			ThreadHelper.JoinableTaskFactory.Run(async () =>
			{
				await ThreadHelper.JoinableTaskFactory.SwitchToMainThreadAsync();
				UpdateCommandLineCombo();
			});
		}

		/// <summary>
		/// Reads options out of the solution file.
		/// </summary>
		public void LoadOptions(Stream Stream)
		{
			using (BinaryReader Reader = new BinaryReader(Stream))
			{
				List<string> CommandLines = new List<string>();
				int Count = Reader.ReadInt32();
				for (int CommandLineIdx = 0; CommandLineIdx < Count; CommandLineIdx++)
				{
					CommandLines.Add(Reader.ReadString());
				}
				ComboList.Clear();
				ComboList.AddRange(CommandLines.ToArray());
			}
		}

		/// <summary>
		/// Writes options to the solution file.
		/// </summary>
		public void SaveOptions(Stream Stream)
		{
			using (BinaryWriter Writer = new BinaryWriter(Stream))
			{
				string[] CommandLines = ComboList.ToArray();
				Writer.Write(CommandLines.Length);
				for (int CommandLineIdx = 0; CommandLineIdx < CommandLines.Length; CommandLineIdx++)
				{
					Writer.Write(CommandLines[CommandLineIdx]);
				}
			}
		}

		/// <summary>
		/// Called when the startup project active config has changed.
		/// </summary>
		private void OnStartupProjectConfigChanged(Project Project)
		{
			ThreadHelper.ThrowIfNotOnUIThread();
			UpdateCommandLineCombo();
		}

		/// <summary>
		/// Updates the command-line combo box after the project loaded state has changed
		/// </summary>
		private void UpdateCommandLineCombo()
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			// Enable or disable our command-line selector
			DesiredCommandLine = null;  // clear this state var used by the combo box handler
			UnrealVSPackage.Instance.SolutionBuildManager.get_StartupProject(out IVsHierarchy ProjectHierarchy);
			if (ProjectHierarchy != null)
			{
				ComboCommand.Enabled = true;
				ComboCommandList.Enabled = true;
			}
			else
			{
				ComboCommand.Enabled = false;
				ComboCommandList.Enabled = false;
			}

			// Mark dirty: MakeCommandLineComboText() will run lazily when VS next polls OutValue.
			// Avoids 5+ COM calls on every OnStartupProjectChanged event (e.g. hover over combo items).
			_CachedCommandLineTextDirty = true;
		}


		/// <summary>
		/// Returns a string to display in the command-line combo box, based on project state
		/// </summary>
		/// <returns>String to display</returns>
		private static string MakeCommandLineComboText()
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			string Text = "";

			if (UnrealVSPackage.Instance.SolutionBuildManager.get_StartupProject(out IVsHierarchy ProjectHierarchy) == VSConstants.S_OK && ProjectHierarchy != null)
			{
				Project SelectedStartupProject = Utils.HierarchyObjectToProject(ProjectHierarchy);
				if (SelectedStartupProject != null)
				{
					Configuration SelectedConfiguration = SelectedStartupProject.ConfigurationManager.ActiveConfiguration;
					if (SelectedConfiguration != null)
					{
						if (ProjectHierarchy is IVsBuildPropertyStorage PropertyStorage)
						{
							// Query the property store for the debugger arguments
							string ConfigurationName = String.Format("{0}|{1}", SelectedConfiguration.ConfigurationName, SelectedConfiguration.PlatformName);
							if (PropertyStorage.GetPropertyValue("LocalDebuggerCommandArguments", ConfigurationName, (uint)_PersistStorageType.PST_USER_FILE, out Text) != VSConstants.S_OK)
							{
								if (PropertyStorage.GetPropertyValue("StartArguments", ConfigurationName, (uint)_PersistStorageType.PST_USER_FILE, out Text) != VSConstants.S_OK)
								{
									Text = "";
								}
							}
							else
							{
								// net core projects move debugger arguments into a debug profile setup via launchSetings.json
								if (PropertyStorage.GetPropertyValue("ActiveDebugProfile", ConfigurationName, (uint)_PersistStorageType.PST_USER_FILE, out string activeDebugProfile) != VSConstants.S_OK)
								{
									activeDebugProfile = null;
								}

								string launchSettingsPath = Path.Combine(Path.GetDirectoryName(SelectedStartupProject.FileName), "Properties", "launchSettings.json");
								if (File.Exists(launchSettingsPath))
								{
									string jsonDoc = File.ReadAllText(launchSettingsPath);
									LaunchSettingsJson settings = JsonSerializer.Deserialize<LaunchSettingsJson>(jsonDoc, new JsonSerializerOptions
									{
										PropertyNameCaseInsensitive = true
									});

									LaunchSettingsJson.LaunchSettingsProfile Profile;
									if (string.IsNullOrEmpty(activeDebugProfile) || !settings.Profiles.TryGetValue(activeDebugProfile, out Profile))
									{
										Profile = settings.Profiles.FirstOrDefault().Value;
									}
									Text = Profile?.CommandLineArgs ?? "";
								}
							}
							// for "Game" projects automatically remove the game project filename from the start of the command line
							var ActiveConfiguration = (SolutionConfiguration2)UnrealVSPackage.Instance.DTE.Solution.SolutionBuild.ActiveConfiguration;
							if (UnrealVSPackage.Instance.IsUESolutionLoaded &&
								Utils.IsGameProject(SelectedStartupProject) &&
								!Utils.IsTestTargetProject(SelectedStartupProject) &&
								Utils.HasUProjectCommandLineArg(ActiveConfiguration.Name))
							{
								string AutoPrefix = Utils.GetAutoUProjectCommandLinePrefix(SelectedStartupProject);
								if (!string.IsNullOrEmpty(AutoPrefix))
								{
									if (Text.Trim().StartsWith(AutoPrefix, StringComparison.OrdinalIgnoreCase))
									{
										Text = Text.Trim().Substring(AutoPrefix.Length).Trim();
									}
								}
							}
						}
					}
				}
			}

			return Text;
		}


		/// Called by combo control to query the text to display or to apply newly-entered text
		private void ComboHandler(object Sender, EventArgs Args)
		{
			ThreadHelper.ThrowIfNotOnUIThread();
			try
			{
				var OleArgs = (OleMenuCmdEventArgs)Args;

				if (OleArgs.InValue is string InString)
				{
					// New text set on the combo - set the command line property
					DesiredCommandLine = null;
					CommitCommandLineText(InString);
					_CachedCommandLineTextDirty = true;
				}
				else if (OleArgs.OutValue != IntPtr.Zero)
				{
					string EditingString = null;
					if (OleArgs.InValue != null)
					{
						if (OleArgs.InValue is object[] InArray && 0 < InArray.Length)
						{
							EditingString = InArray.Last() as string;
						}
					}

					string TextToDisplay = string.Empty;
					if (EditingString != null)
					{
						// The control wants to put EditingString in the box
						TextToDisplay = DesiredCommandLine = EditingString;
					}
					else
					{
						// This is always hit at the end of interaction with the combo
						if (DesiredCommandLine != null)
						{
							TextToDisplay = DesiredCommandLine;
							DesiredCommandLine = null;
							CommitCommandLineText(TextToDisplay);
							_CachedCommandLineTextDirty = true;
						}
						else
						{
							if (_CachedCommandLineTextDirty)
							{
								string NewText = MakeCommandLineComboText();
								if (NewText != CachedCommandLineText)
								{
									CachedCommandLineText = NewText;
									CommitCommandLineToMRU(CachedCommandLineText);
								}
								_CachedCommandLineTextDirty = false;
							}
							TextToDisplay = CachedCommandLineText;
						}
					}

					Marshal.GetNativeVariantForObject(TextToDisplay, OleArgs.OutValue);
				}
			}
			catch (Exception ex)
			{
				Exception AppEx = new ApplicationException("CommandLineEditor threw an exception in ComboHandler()", ex);
				Logging.WriteLine(AppEx.ToString());
				throw AppEx;
			}
		}

		private void CommitCommandLineText(string CommandLine)
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			if (UnrealVSPackage.Instance.SolutionBuildManager.get_StartupProject(out IVsHierarchy ProjectHierarchy) == VSConstants.S_OK && ProjectHierarchy != null)
			{
				Project SelectedStartupProject = Utils.HierarchyObjectToProject(ProjectHierarchy);
				if (SelectedStartupProject != null)
				{
					Configuration SelectedConfiguration = SelectedStartupProject.ConfigurationManager.ActiveConfiguration;
					if (SelectedConfiguration != null)
					{
						if (ProjectHierarchy is IVsBuildPropertyStorage PropertyStorage)
						{
							string FullCommandLine = CommandLine;

							// for "Game" projects automatically remove the game project filename from the start of the command line
							var ActiveConfiguration = (SolutionConfiguration2)UnrealVSPackage.Instance.DTE.Solution.SolutionBuild.ActiveConfiguration;
							if (UnrealVSPackage.Instance.IsUESolutionLoaded &&
								Utils.IsGameProject(SelectedStartupProject) &&
								!Utils.IsTestTargetProject(SelectedStartupProject) &&
								Utils.HasUProjectCommandLineArg(ActiveConfiguration.Name))
							{
								string AutoPrefix = Utils.GetAutoUProjectCommandLinePrefix(SelectedStartupProject);
								if (FullCommandLine.IndexOf(Utils.UProjectExtension, StringComparison.OrdinalIgnoreCase) < 0 &&
									string.Compare(FullCommandLine, SelectedStartupProject.Name, StringComparison.OrdinalIgnoreCase) != 0 &&
									!FullCommandLine.StartsWith(SelectedStartupProject.Name + " ", StringComparison.OrdinalIgnoreCase))
								{
									// Committed command line does not specify a .uproject
									FullCommandLine = AutoPrefix + " " + FullCommandLine;
								}
							}

							// Get the project platform name.
							string ProjectPlatformName = SelectedConfiguration.PlatformName;
							if (ProjectPlatformName == "Any CPU")
							{
								ProjectPlatformName = "AnyCPU";
							}

							// Get the project kind. C++ projects store the debugger arguments differently to other project types.
							string ProjectKind = SelectedStartupProject.Kind;

							// Update the property
							string ProjectConfigurationName = String.Format("{0}|{1}", SelectedConfiguration.ConfigurationName, ProjectPlatformName);
							if (String.Equals(ProjectKind, "{8BC9CEB8-8B4A-11D0-8D11-00A0C91BC942}", StringComparison.InvariantCultureIgnoreCase))
							{
								List<string> ExtraFields = Utils.GetExtraDebuggerCommandArguments(ProjectPlatformName, SelectedStartupProject);

								if (FullCommandLine.Length == 0)
								{
									PropertyStorage.RemoveProperty("LocalDebuggerCommandArguments", ProjectConfigurationName, (uint)_PersistStorageType.PST_USER_FILE);
									PropertyStorage.RemoveProperty("RemoteDebuggerCommandArguments", ProjectConfigurationName, (uint)_PersistStorageType.PST_USER_FILE);
									foreach (string ExtraField in ExtraFields)
									{
										PropertyStorage.RemoveProperty(ExtraField, ProjectConfigurationName, (uint)_PersistStorageType.PST_USER_FILE);
									}
								}
								else
								{
									PropertyStorage.SetPropertyValue("LocalDebuggerCommandArguments", ProjectConfigurationName, (uint)_PersistStorageType.PST_USER_FILE, FullCommandLine);
									PropertyStorage.SetPropertyValue("RemoteDebuggerCommandArguments", ProjectConfigurationName, (uint)_PersistStorageType.PST_USER_FILE, FullCommandLine);
									foreach (string ExtraField in ExtraFields)
									{
										PropertyStorage.SetPropertyValue(ExtraField, ProjectConfigurationName, (uint)_PersistStorageType.PST_USER_FILE, FullCommandLine);
									}
								}
							}
							else
							{
								// For some reason, have to update C# projects this way, otherwise the properties page doesn't update. Conversely, SelectedConfiguration.Properties is always null for C++ projects in VS2017.
								bool netCoreProject = false;
								if (SelectedConfiguration.Properties != null)
								{
									foreach (Property Property in SelectedConfiguration.Properties)
									{
										if (Property.Name.Equals("StartArguments", StringComparison.InvariantCultureIgnoreCase))
										{
											try
											{
												Property.Value = FullCommandLine;
											}
											catch (NotImplementedException)
											{
												// net core projects report containing start arguments command but does not support setting it
												netCoreProject = true;
											}
											break;
										}
									}
								}

								if (netCoreProject)
								{
									// net core project use launchSettings.json to control debug arguments instead

									if (PropertyStorage.GetPropertyValue("ActiveDebugProfile", ProjectConfigurationName, (uint)_PersistStorageType.PST_USER_FILE, out string activeDebugProfile) != VSConstants.S_OK)
									{
										activeDebugProfile = null;
									}

									string launchSettingsDir = Path.Combine(Path.GetDirectoryName(SelectedStartupProject.FileName), "Properties");
									string launchSettingsPath = Path.Combine(launchSettingsDir, "launchSettings.json");
									LaunchSettingsJson settings = null;
									if (File.Exists(launchSettingsPath))
									{
										try
										{
											string jsonDoc = File.ReadAllText(launchSettingsPath);
											settings = JsonSerializer.Deserialize<LaunchSettingsJson>(jsonDoc, new JsonSerializerOptions
											{
												PropertyNameCaseInsensitive = true
											});
										}
										catch (Exception Ex)
										{
											Logging.WriteLine($"Error loading {launchSettingsPath}");
											Logging.WriteLine(Ex.ToString());
										}
									}

									// Create empty Launch Settings if launchSettings.json does not exist or failed to load
									if (settings == null || settings.Profiles == null || settings.Profiles.Count == 0)
									{
										settings = new LaunchSettingsJson
										{
											Profiles = new Dictionary<string, LaunchSettingsJson.LaunchSettingsProfile>()
										};
										settings.Profiles.Add(SelectedStartupProject.Name, new LaunchSettingsJson.LaunchSettingsProfile
										{
											CommandName = "Project",
											CommandLineArgs = string.Empty,
										});
									}

									if (!string.IsNullOrEmpty(activeDebugProfile) && settings.Profiles.ContainsKey(activeDebugProfile))
									{
										settings.Profiles[activeDebugProfile].CommandLineArgs = FullCommandLine;
									}
									else
									{
										// if no active debug profile is set then VS uses the first one
										settings.Profiles.First().Value.CommandLineArgs = FullCommandLine;
									}

									string json = JsonSerializer.Serialize(settings, new JsonSerializerOptions
									{
										WriteIndented = true,
									});

									Directory.CreateDirectory(launchSettingsDir);
									File.WriteAllText(launchSettingsPath, json);
								}
							}
						}
					}
				}
			}

			CommitCommandLineToMRU(CommandLine);
		}

		private void CommitCommandLineToMRU(string CommandLine)
		{
			if (0 < CommandLine.Length)
			{
				// Maintain the MRU history
				// Adds the entered command line to the top of the list
				// Moves it to the top if it's already in the list
				// Trims the list to the max length if it's too big
				ComboList.RemoveAll(s => 0 == string.Compare(s, CommandLine));
				ComboList.Insert(0, CommandLine);
				if (ComboList.Count > ComboListCountMax)
				{
					ComboList.RemoveAt(ComboList.Count - 1);
				}
			}
		}

		/// Called by combo control to populate the drop-down list
		private void ComboListHandler(object Sender, EventArgs Args)
		{
			var OleArgs = (OleMenuCmdEventArgs)Args;

			Marshal.GetNativeVariantForObject(ComboList.ToArray(), OleArgs.OutValue);
		}

		/// List of MRU strings to show in the combobox drop-down list
		private readonly List<string> ComboList = new List<string>();

		/// Combo control for command-line editor
		private OleMenuCommand ComboCommand;
		private OleMenuCommand ComboCommandList;

		/// Used to store the user edited commandline mid-edit, in the combo handler
		private string DesiredCommandLine;

		/// Cached result of MakeCommandLineComboText(). Lazily computed: set dirty on state change,
		/// recomputed only when VS polls OutValue. Avoids expensive COM calls on every project change event.
		private string CachedCommandLineText = string.Empty;
		private bool _CachedCommandLineTextDirty = true;

		/// used to shield against too many property changes in the future
		private System.Timers.Timer UpdateCommandLineComboTimer = null;

		// Track IDisposable state
		private bool IsDisposed;

		protected virtual void Dispose(bool disposing)
		{
			if (!IsDisposed)
			{
				if (disposing)
				{
					// Dispose managed state (managed objects)
					UpdateCommandLineComboTimer?.Dispose();
					UpdateCommandLineComboTimer = null;
				}

				// Free unmanaged resources (unmanaged objects) and override finalizer
				// Set large fields to null
				IsDisposed = true;
			}
		}

		public void Dispose()
		{
			// Do not change this code. Put cleanup code in 'Dispose(bool disposing)' method
			Dispose(disposing: true);
			GC.SuppressFinalize(this);
		}
	}
}
