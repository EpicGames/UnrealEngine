// Copyright Epic Games, Inc. All Rights Reserved.

using EnvDTE;
using Microsoft.VisualStudio.Shell;
using Microsoft.VisualStudio.Shell.Interop;
using System;
using System.Linq;
using System.Windows.Forms;

namespace UnrealVS
{
	internal class CompileScoreHelper
	{
		/// <summary>
		/// Checks if the CompileScore plugin's commands have been registered.
		/// </summary>
		/// <returns>True if the plugin is installed.</returns>
		public static bool IsInstalled()
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			Command Command = UnrealVSPackage.Instance.DTE.Commands
				.Cast<Command>()
				.FirstOrDefault(C =>
				{
					ThreadHelper.ThrowIfNotOnUIThread();
					return C.Name == "CompileScore.StartTrace";
				});

			return Command != null && Command.IsAvailable;
		}

		public static void AlertNotInstalled()
		{
			MessageBox.Show($"This action requires a CompileScore Visual Studio extension that has CompileScore.StartTrace/CompileScore.StopTrace installed", "UnrealVS - Missing CompileScore extension ", MessageBoxButtons.OK);
		}

		/// <summary>
		/// Start a CompileScore trace through the plugin's Visual Studio commands.
		/// If the plugin is not installed, this will alert the user.
		/// </summary>
		/// <returns>True if the trace has been started.</returns>
		public static bool StartTrace()
		{
			if (!IsInstalled())
			{
				AlertNotInstalled();
				return false;
			}

			ThreadHelper.ThrowIfNotOnUIThread();

			UnrealVSPackage.Instance.DTE.ExecuteCommand("CompileScore.StartTrace");

			IVsOutputWindowPane BuildOutputPane = UnrealVSPackage.Instance.GetOutputPane();
			UnrealVSPackage.Instance.GetOutputPane()?.OutputStringThreadSafe($"1>------ Started trace for CompileScore ------{Environment.NewLine}");

			return true;
		}

		/// <summary>
		/// Stop a CompileScore trace through the plugin's Visual Studio commands.
		/// </summary>
		public static void StopTrace()
		{
			ThreadHelper.ThrowIfNotOnUIThread();
			UnrealVSPackage.Instance.DTE.ExecuteCommand("CompileScore.StopTrace");

			UnrealVSPackage.Instance.GetOutputPane()?.OutputStringThreadSafe($"1>------ Stopped trace for CompileScore ------{Environment.NewLine}");
		}
	}
}
