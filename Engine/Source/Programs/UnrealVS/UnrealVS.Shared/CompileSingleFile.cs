// Copyright Epic Games, Inc. All Rights Reserved.

using EnvDTE;
using Microsoft.VisualStudio.Shell;
using Microsoft.VisualStudio.Shell.Interop;
using System;
using System.Collections.Generic;
using System.ComponentModel.Design;
using System.IO;
using System.Linq;
using System.Windows.Forms;

namespace UnrealVS
{
	class CompileSingleFile : IDisposable
	{
		private const int CompileSingleFileButtonID = 0x1075;
		private const int PreprocessSingleFileButtonID = 0x1076;
		private const int CompileSingleModuleButtonID = 0x1077;
		private const int CompileAndProfileSingleFileButtonID = 0x1078;
		private const int GenerateAssemblyFileButtonID = 0x1079;
		private const int UBTSubMenuID = 0x3103;

		private static readonly HashSet<string> ValidExtensions = new HashSet<string>(StringComparer.OrdinalIgnoreCase) { ".c", ".cc", ".cpp", ".h", ".cxx", ".ispc" };

		private UnrealProjectBuilder ProjectBuilder = new UnrealProjectBuilder();

		public CompileSingleFile()
		{
			AddCommandInternal(CompileSingleFileButtonID);
			AddCommandInternal(PreprocessSingleFileButtonID);
			AddCommandInternal(CompileSingleModuleButtonID);
			AddCommandInternal(CompileAndProfileSingleFileButtonID);
			AddCommandInternal(GenerateAssemblyFileButtonID);

			// add sub menu for UBT commands
			UnrealVSPackage.Instance.MenuCommandService.AddCommand(new OleMenuCommand(null, new CommandID(GuidList.UnrealVSCmdSet, UBTSubMenuID)));

			void AddCommandInternal(int CommandID)
			{
				var MenuCommand = new OleMenuCommand(CompileSingleFileButtonHandler, new CommandID(GuidList.UnrealVSCmdSet, CommandID));
				MenuCommand.BeforeQueryStatus += CompileSingleFileButtonCommand_BeforeQueryStatus;
				UnrealVSPackage.Instance.MenuCommandService.AddCommand(MenuCommand);
			}
		}

		public void Dispose()
		{
			ProjectBuilder.Dispose();
		}

		private void CompileSingleFileButtonCommand_BeforeQueryStatus(object Sender, EventArgs e)
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			if (Sender is MenuCommand MenuCommand)
			{
				DTE DTE = UnrealVSPackage.Instance.DTE;

				MenuCommand.Enabled =
					DTE?.ActiveDocument != null
					// Check if compile score is required and installed
					&& (MenuCommand.CommandID.ID != CompileAndProfileSingleFileButtonID || CompileScoreHelper.IsInstalled())
					// Check if the requested file is valid
					&& ValidExtensions.Contains(Path.GetExtension(DTE.ActiveDocument.FullName));
			}
		}

		void CompileSingleFileButtonHandler(object Sender, EventArgs Args)
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			MenuCommand SenderSubMenuCommand = (MenuCommand)Sender;

			bool bIsFile = SenderSubMenuCommand.CommandID.ID != CompileSingleModuleButtonID;
			bool bPreprocessOnly = SenderSubMenuCommand.CommandID.ID == PreprocessSingleFileButtonID;
			bool bProfile = SenderSubMenuCommand.CommandID.ID == CompileAndProfileSingleFileButtonID;
			bool bGenerateAssembly = SenderSubMenuCommand.CommandID.ID == GenerateAssemblyFileButtonID;

			if (!TryCompileSingleFileOrModule(bIsFile, bPreprocessOnly, bProfile, bGenerateAssembly))
			{
				if (bProfile && !CompileScoreHelper.StartTrace())
				{
					return;
				}

				DTE DTE = UnrealVSPackage.Instance.DTE;
				DTE.ExecuteCommand("Build.Compile");

				if (bProfile)
				{
					var WaitThread = new System.Threading.Thread(() =>
					{
						bool Done = false;
						while (!Done)
						{
							ThreadHelper.JoinableTaskFactory.Run(async () =>
							{
								await ThreadHelper.JoinableTaskFactory.SwitchToMainThreadAsync();
								if (DTE.Solution.SolutionBuild.BuildState != vsBuildState.vsBuildStateDone)
								{
									return;
								}
								CompileScoreHelper.StopTrace();
								Done = true;

							});
						}
					})
					{ Priority = System.Threading.ThreadPriority.Lowest };
					WaitThread.Start();
				}
			}
		}

		static public string FindModuleForFile(string FileName, string RootDirectory)
		{
			string ModuleBuildFilePath = FindBuildFilePathForFile(FileName, RootDirectory);
			if (ModuleBuildFilePath != null)
			{
				// *.Build.cs -> *.Build -> *
				return Path.GetFileNameWithoutExtension(Path.GetFileNameWithoutExtension(ModuleBuildFilePath));
			}
			return null;
		}

		static public string FindBuildFilePathForFile(string FileName, string RootDirectory)
		{
			for (string Dir = Path.GetDirectoryName(FileName);
				Dir != null && !String.Equals(Dir, RootDirectory, StringComparison.OrdinalIgnoreCase);
				Dir = Path.GetDirectoryName(Dir))
			{
				string BuildFile = Directory.EnumerateFiles(Dir, "*.Build.cs", SearchOption.TopDirectoryOnly).FirstOrDefault();
				if (BuildFile != null)
				{
					return BuildFile;
				}
			}
			return null;
		}

		bool TryCompileSingleFileOrModule(bool bIsFile, bool bPreProcessOnly, bool bProfile, bool bGenerateAssembly)
		{
			ThreadHelper.ThrowIfNotOnUIThread();
			DTE DTE = UnrealVSPackage.Instance.DTE;

			// Check we've got a file open
			if (DTE.ActiveDocument == null)
			{
				Logging.WriteLine("CompileSingleFile: ActiveDocument not found");
				return false;
			}

			// Grab the current startup project
			UnrealVSPackage.Instance.SolutionBuildManager.get_StartupProject(out IVsHierarchy ProjectHierarchy);
			if (ProjectHierarchy == null)
			{
				Logging.WriteLine("CompileSingleFile: ProjectHierarchy not found");
				return false;
			}

			if (!(Utils.HierarchyObjectToProject(ProjectHierarchy) is Project StartupProject))
			{
				Logging.WriteLine("CompileSingleFile: StartupProject not found");
				return false;
			}

			// Check if the requested file is valid
			string FileToCompile = DTE.ActiveDocument.FullName;
			string FileToCompileExt = Path.GetExtension(FileToCompile);

			List<string> ExtraArguments = new List<string>();

			if (bPreProcessOnly)
			{
				ExtraArguments.Add("-NoXGE -NoSNDBS -NoFASTBuild");
				ExtraArguments.Add("-Preprocess");
			}
			else if (bGenerateAssembly)
			{
				ExtraArguments.Add("-NoXGE -NoSNDBS -NoFASTBuild");
				ExtraArguments.Add("-WithAssembly");
			}

			if (bIsFile)
			{
				if (!ValidExtensions.Contains(FileToCompileExt.ToLowerInvariant()))
				{
					MessageBox.Show($"Invalid file extension {FileToCompileExt} for single-file compile.", "Invalid Extension", MessageBoxButtons.OK);
					return true;
				}

				return ProjectBuilder.BuildSingleFile(StartupProject, FileToCompile, ExtraArguments, bProfile);
			}
			else
			{
				string ModuleName = FindModuleForFile(FileToCompile, Path.GetDirectoryName(DTE.Solution.FileName));
				if (ModuleName == null)
				{
					MessageBox.Show($"Can't find module for for {FileToCompile} to compile.", "Invalid Module", MessageBoxButtons.OK);
					return true;
				}
				return ProjectBuilder.BuildModule(StartupProject, ModuleName, ExtraArguments, bProfile);
			}
		}
	}
}
