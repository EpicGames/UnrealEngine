// Copyright Epic Games, Inc. All Rights Reserved.

using EnvDTE;
using Microsoft.VisualStudio.Shell;
using System;
using System.Collections.Generic;
using System.ComponentModel.Design;
using System.IO;
using System.Windows.Forms;

namespace UnrealVS
{
	class CopyIncludePath : IDisposable
	{
		private const int CopyIncludePathButtonID = 0x1470;

		/// <summary>
		/// Well-known module subdirectory names that serve as include roots in UE.
		/// When a header lives under one of these directories the include path
		/// starts immediately after it (e.g. "MyModule/Public/Foo/Bar.h" becomes
		/// "Foo/Bar.h").
		/// </summary>
		private static readonly HashSet<string> IncludeRootFolders =
			new HashSet<string>(StringComparer.OrdinalIgnoreCase)
			{
				"Public",
				"Private",
				"Classes",
				"Internal",
			};

		private readonly OleMenuCommand MenuCommand;

		public CopyIncludePath()
		{
			MenuCommand = new OleMenuCommand(OnExecute, new CommandID(GuidList.UnrealVSCmdSet, CopyIncludePathButtonID));
			MenuCommand.BeforeQueryStatus += OnBeforeQueryStatus;
			UnrealVSPackage.Instance.MenuCommandService.AddCommand(MenuCommand);
		}

		public void Dispose()
		{
		}

		/// <summary>
		/// Only show the menu item when the active document is a valid Unreal header
		/// (a .h file inside a UE solution that lives under a known module include root).
		/// </summary>
		private void OnBeforeQueryStatus(object sender, EventArgs e)
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			bool bVisible = false;
			DTE DTE = UnrealVSPackage.Instance.DTE;

			if (UnrealVSPackage.Instance.IsUESolutionLoaded
				&& DTE?.ActiveDocument != null
				&& Path.GetExtension(DTE.ActiveDocument.FullName).Equals(".h", StringComparison.OrdinalIgnoreCase))
			{
				bVisible = BuildIncludePath(DTE.ActiveDocument.FullName, out _);
			}

			MenuCommand.Visible = bVisible;
			MenuCommand.Enabled = bVisible;
		}

		/// <summary>
		/// Build a #include "..." string from the active document path and copy it
		/// to the clipboard.
		/// </summary>
		private void OnExecute(object sender, EventArgs e)
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			DTE DTE = UnrealVSPackage.Instance.DTE;
			if (DTE?.ActiveDocument == null)
			{
				return;
			}

			string FullPath = DTE.ActiveDocument.FullName;
			if (!BuildIncludePath(FullPath, out string IncludePath))
			{
				return;
			}

			string IncludeDirective = $"#include \"{IncludePath}\"";

			Clipboard.SetText(IncludeDirective);

			// Brief status bar message so the user knows it worked
			DTE.StatusBar.Text = $"Copied: {IncludeDirective}";
		}

		/// <summary>
		/// Determines the shortest conventional UE #include path for a header.
		///
		/// Strategy: walk up from the file looking for a well-known include-root
		/// folder (Public, Private, Classes, Internal). The include path is
		/// everything after that folder.
		///
		/// Returns false if no include root folder is found in the path.
		/// </summary>
		internal static bool BuildIncludePath(string FullPath, out string IncludePath)
		{
			FullPath = Path.GetFullPath(FullPath);

			// Collect path segments from the file upward
			List<string> Segments = new List<string>();
			string Dir = Path.GetDirectoryName(FullPath);
			Segments.Add(Path.GetFileName(FullPath));

			while (!string.IsNullOrEmpty(Dir))
			{
				string FolderName = Path.GetFileName(Dir);

				if (IncludeRootFolders.Contains(FolderName))
				{
					// Everything we've accumulated so far (in reverse) is the include path
					Segments.Reverse();
					IncludePath = string.Join("/", Segments);
					return true;
				}

				Segments.Add(FolderName);
				string Parent = Path.GetDirectoryName(Dir);
				if (Parent == Dir)
				{
					break; // root
				}
				Dir = Parent;
			}

			IncludePath = null;
			return false;
		}
	}
}
