// Copyright Epic Games, Inc. All Rights Reserved.

using EnvDTE;
using Microsoft.VisualStudio.Shell;
using Microsoft.VisualStudio.Shell.Interop;
using System;
using System.ComponentModel.Design;
using System.IO;

namespace UnrealVS
{
	/// Cycles through files that share the same base name as the active document
	/// (e.g. MyClass.h → MyClass.cpp → MyClass.inl → …).
	/// The search root is the module directory found by walking up from the current
	/// file and locating the folder that contains a .build.cs file.
	class CycleRelatedFiles
	{
		const int CycleRelatedFilesButtonID = 0x1339;

		public CycleRelatedFiles()
		{
			CommandID CommandID = new CommandID(GuidList.UnrealVSCmdSet, CycleRelatedFilesButtonID);
			MenuCommand Command = new MenuCommand(new EventHandler(CycleRelatedFilesHandler), CommandID);
			UnrealVSPackage.Instance.MenuCommandService.AddCommand(Command);
		}

		[global::System.Diagnostics.CodeAnalysis.SuppressMessageAttribute("", "VSTHRD010")]
		private void CycleRelatedFilesHandler(object Sender, EventArgs Args)
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			Document ActiveDoc = UnrealVSPackage.Instance.DTE.ActiveDocument;
			if (ActiveDoc == null)
			{
				Logging.WriteLine("CycleRelatedFiles: ActiveDocument not found");
				return;
			}

			string CurrentFilePath = ActiveDoc.FullName;
			string BaseFileName = Path.GetFileNameWithoutExtension(CurrentFilePath);
			string CurrentDir = Path.GetDirectoryName(CurrentFilePath);

			// Find the module root: walk up until we find a directory containing a .build.cs file.
			string SolutionFileName = UnrealVSPackage.Instance.DTE.Solution.FileName;
			if (SolutionFileName == null)
			{
				Logging.WriteLine("CycleRelatedFiles: Solution not found");
				return;
			}
			string ModuleRoot = CompileSingleFile.FindBuildFilePathForFile(CurrentFilePath, Path.GetDirectoryName(SolutionFileName));
			if (string.IsNullOrEmpty(ModuleRoot))
			{
				ModuleRoot = CurrentDir;
			}
			else
			{
				ModuleRoot = Path.GetDirectoryName(ModuleRoot);
			}

			// Collect all files with the same base name anywhere under the module root.
			string[] RelatedFiles;
			try
			{
				RelatedFiles = Directory.GetFiles(ModuleRoot, BaseFileName + ".*", SearchOption.AllDirectories);
			}
			catch
			{
				Logging.WriteLine("CycleRelatedFiles: Inaccessible directory");
				return;
			}
			if (RelatedFiles.Length <= 1)
			{
				Logging.WriteLine("CycleRelatedFiles: No other files");
				return;
			}

			Array.Sort(RelatedFiles, StringComparer.OrdinalIgnoreCase);

			int CurrentIndex = Array.FindIndex(RelatedFiles,
				F => string.Equals(F, CurrentFilePath, StringComparison.OrdinalIgnoreCase));

			// If not found, then CurrentIndex will be -1 , and NextIndex will wrap to 0, which is what we want.
			int NextIndex = (CurrentIndex + 1) % RelatedFiles.Length;

			VsShellUtilities.OpenDocument(UnrealVSPackage.Instance, RelatedFiles[NextIndex]);
		}
	}
}
