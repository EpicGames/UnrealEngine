// Copyright Epic Games, Inc. All Rights Reserved.

using EnvDTE80;
using Microsoft.VisualStudio.Shell;
using Microsoft.VisualStudio.Shell.Interop;
using System;
using System.ComponentModel.Design;
using System.Runtime.InteropServices;
using System.Windows.Forms;

namespace UnrealVS
{
	class FileBrowser
	{
		const int FileBrowserButtonID = 0x1337;

		public FileBrowser()
		{
			// FileBrowserButton
			var CommandID = new CommandID(GuidList.UnrealVSCmdSet, FileBrowserButtonID);
			var FileBrowserButtonCommand = new MenuCommand(new EventHandler(FileBrowserButtonHandler), CommandID);
			UnrealVSPackage.Instance.MenuCommandService.AddCommand(FileBrowserButtonCommand);

			UnrealVSPackage.Instance.OnSolutionOpened += () => BrowserControl?.HandleSolutionChanged();
			UnrealVSPackage.Instance.OnSolutionClosed += () => BrowserControl?.HandleSolutionChanged();
			UnrealVSPackage.Instance.OnProjectLoaded += (p) => BrowserControl?.HandleSolutionChanged();
			UnrealVSPackage.Instance.OnProjectUnloading += (p) => BrowserControl?.HandleSolutionChanged();
			UnrealVSPackage.Instance.OnDocumentActivated += (d) => BrowserControl?.HandleDocumentActivated(d);

		}

		/// Called when 'FileBrowser' button is clicked
		void FileBrowserButtonHandler(object Sender, EventArgs Args)
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			ToolWindowPane ToolWindow = UnrealVSPackage.Instance.FindToolWindow(typeof(FileBrowserWindow), 0, true);
			if ((null == ToolWindow) || (null == ToolWindow.Frame))
			{
				throw new NotSupportedException(Resources.ToolWindowCreateError);
			}
			IVsWindowFrame ToolWindowFrame = (IVsWindowFrame)ToolWindow.Frame;
			Microsoft.VisualStudio.ErrorHandler.ThrowOnFailure(ToolWindowFrame.Show());

			// IsVisibleChanged only fires when visibility actually changes; if the window
			// was already visible, focus the filter box explicitly every time.
			BrowserControl?.HandleShow();
		}

		FileBrowserWindowControl BrowserControl
		{
			get
			{
				ToolWindowPane ToolWindow = UnrealVSPackage.Instance.FindToolWindow(typeof(FileBrowserWindow), 0, true);
				if ((null == ToolWindow) || (null == ToolWindow.Frame))
					return null;
				var fileBrowser = (FileBrowserWindow)ToolWindow;
				return (FileBrowserWindowControl)fileBrowser.Content;
			}
		}
	}

	[Guid("b205a03c-566f-4dc3-9e2f-99e2df765f1d")]
	public class FileBrowserWindow : ToolWindowPane
	{
		public FileBrowserWindow() : base(null)
		{
			this.Caption = "Unreal File Browser (F1 for help)";

			this.Content = new FileBrowserWindowControl(this);
		}

		[global::System.Diagnostics.CodeAnalysis.SuppressMessageAttribute("", "VSTHRD010")]
		protected override bool PreProcessMessage(ref Message m)
        {
			if (m.Msg == 0x0100)//WM_KEYDOWN)
			{
				if (m.WParam == new IntPtr(0x1B)) // VK_ESCAPE
				{
					((FileBrowserWindowControl)this.Content).HandleEscape();
					return true;
				}
				else if (m.WParam == new IntPtr(0x70)) // VK_F1
				{
					((FileBrowserWindowControl)this.Content).HandleF1();
					return true;
				}
				else if (m.WParam == new IntPtr(0x74)) // VK_F5
                {
					((FileBrowserWindowControl)this.Content).HandleF5();
					return true;
                }
			}
			return base.PreProcessMessage(ref m);
        }
    }

}
