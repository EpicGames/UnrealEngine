<UserControl x:Class="UnrealVS.FileBrowserWindowControl"
             xmlns="http://schemas.microsoft.com/winfx/2006/xaml/presentation"
             xmlns:x="http://schemas.microsoft.com/winfx/2006/xaml"
             xmlns:mc="http://schemas.openxmlformats.org/markup-compatibility/2006"
             xmlns:d="http://schemas.microsoft.com/expression/blend/2008"
             xmlns:vsshell="clr-namespace:Microsoft.VisualStudio.Shell;assembly=Microsoft.VisualStudio.Shell.15.0"
             Background="{DynamicResource {x:Static vsshell:VsBrushes.ToolWindowBackgroundKey}}"
             Foreground="{DynamicResource {x:Static vsshell:VsBrushes.WindowTextKey}}"
             mc:Ignorable="d"
             d:DesignHeight="300" d:DesignWidth="500"
             Name="FileBrowser">
    <UserControl.Resources>
        <Style TargetType="TextBox">
            <Setter Property="Background" Value="Transparent"/>
            <Setter Property="Foreground" Value="{DynamicResource {x:Static vsshell:VsBrushes.CommandBarTextActiveKey}}"/>
        </Style>
        <Style TargetType="ListBox">
            <Setter Property="Background" Value="Transparent"/>
            <Setter Property="BorderThickness" Value="1"/>
            <Setter Property="Padding" Value="5" />
            <Setter Property="BorderBrush" Value="{DynamicResource {x:Static vsshell:VsBrushes.ToolWindowBorderKey}}"/>
            <Setter Property="Margin" Value="0,-6,0,0"/>
            <Setter Property="FocusVisualStyle" Value="{x:Null}"/>
        </Style>
        <Style TargetType="ListView">
            <Setter Property="Background" Value="Transparent"/>
            <Setter Property="BorderThickness" Value="1"/>
            <Setter Property="BorderBrush" Value="{DynamicResource {x:Static vsshell:VsBrushes.ToolWindowBorderKey}}"/>
            <Setter Property="Margin" Value="0,-6,0,0"/>
            <Setter Property="FocusVisualStyle" Value="{x:Null}"/>
            <Setter Property="ScrollViewer.HorizontalScrollBarVisibility" Value="Auto"/>
        </Style>
        <Style TargetType="TabItem">
            <Setter Property="Background" Value="Transparent"/>
            <Setter Property="IsTabStop" Value="False"/>
            <Setter Property="Foreground" Value="{DynamicResource {x:Static vsshell:VsBrushes.WindowTextKey}}"/>
            <Setter Property="FocusVisualStyle" Value="{StaticResource ListBoxItemVisualStyle}" />
            <Setter Property="Template">
                <Setter.Value>
                    <ControlTemplate TargetType="TabItem">
                        <Border Name="Border" BorderThickness="1,1,1,0" BorderBrush="{DynamicResource {x:Static vsshell:VsBrushes.ToolWindowBorderKey}}" CornerRadius="2,2,0,0" Margin="1,2,2,2">
                            <ContentPresenter x:Name="ContentSite"
                                    VerticalAlignment="Center"
                                    HorizontalAlignment="Center"
                                    ContentSource="Header"
                                    Margin="10,3"/>
                        </Border>
                        <ControlTemplate.Triggers>
                            <Trigger Property="IsSelected" Value="True">
                                <Setter TargetName="Border" Property="Background" Value="{DynamicResource {x:Static vsshell:VsBrushes.WindowKey}}" />
                            </Trigger>
                            <Trigger Property="IsSelected" Value="False">
                                <Setter TargetName="Border" Property="Background" Value="Transparent" />
                                <Setter TargetName="ContentSite" Property="TextElement.Foreground" Value="{DynamicResource {x:Static vsshell:VsBrushes.FileTabInactiveTextKey}}" />
                            </Trigger>
                        </ControlTemplate.Triggers>
                    </ControlTemplate>
                </Setter.Value>
            </Setter>
        </Style>
        <Style x:Key="ListBoxItemVisualStyle" TargetType="Control">
        </Style>

        <Style TargetType="ListBoxItem">
            <Setter Property="Foreground" Value="{DynamicResource {x:Static vsshell:VsBrushes.CommandBarTextHoverKey}}"/>
            <Setter Property="Template">
                <Setter.Value>
                    <ControlTemplate TargetType="{x:Type ListBoxItem}">
                        <Border BorderBrush="Transparent" BorderThickness="0" Background="{TemplateBinding Background}">
                            <ContentPresenter>
                                <ContentPresenter.Resources>
                                    <Style TargetType="TextBlock">
                                        <Setter Property="Margin" Value="2"/>
                                    </Style>
                                </ContentPresenter.Resources>
                            </ContentPresenter>
                        </Border>
                    </ControlTemplate>
                </Setter.Value>
            </Setter>
            <Style.Triggers>
                <MultiTrigger>
                    <MultiTrigger.Conditions>
                        <Condition Property="IsSelected" Value="true" />
                    </MultiTrigger.Conditions>
                    <Setter Property="Background" Value="{DynamicResource {x:Static vsshell:VsBrushes.ToolWindowTabMouseOverBackgroundBeginKey}}" />
                    <Setter Property="FocusVisualStyle" Value="{StaticResource ListBoxItemVisualStyle}" />
                </MultiTrigger>
                <MultiDataTrigger>
                    <MultiDataTrigger.Conditions>
                        <Condition Binding="{Binding Path=Name}" Value="" />
                    </MultiDataTrigger.Conditions>
                    <Setter Property="Background" Value="{x:Null}" />
                </MultiDataTrigger>
            </Style.Triggers>
        </Style>
        <Style TargetType="ListViewItem">
            <Setter Property="Foreground" Value="{DynamicResource {x:Static vsshell:VsBrushes.CommandBarTextHoverKey}}"/>
            <Setter Property="Template">
                <Setter.Value>
                    <ControlTemplate TargetType="{x:Type ListViewItem}">
                        <Border BorderBrush="Transparent" BorderThickness="0" Background="{TemplateBinding Background}">
                            <GridViewRowPresenter Margin="2,1"/>
                        </Border>
                    </ControlTemplate>
                </Setter.Value>
            </Setter>
            <Style.Triggers>
                <MultiTrigger>
                    <MultiTrigger.Conditions>
                        <Condition Property="IsSelected" Value="true" />
                    </MultiTrigger.Conditions>
                    <Setter Property="Background" Value="{DynamicResource {x:Static vsshell:VsBrushes.ToolWindowTabMouseOverBackgroundBeginKey}}" />
                    <Setter Property="FocusVisualStyle" Value="{StaticResource ListBoxItemVisualStyle}" />
                </MultiTrigger>
                <MultiDataTrigger>
                    <MultiDataTrigger.Conditions>
                        <Condition Binding="{Binding Path=Name}" Value="" />
                    </MultiDataTrigger.Conditions>
                    <Setter Property="Background" Value="{x:Null}" />
                </MultiDataTrigger>
            </Style.Triggers>
        </Style>
        <Style TargetType="{x:Type GridViewColumnHeader}">
            <Setter Property="Background" Value="{DynamicResource {x:Static vsshell:VsBrushes.ToolWindowBackgroundKey}}"/>
            <Setter Property="Foreground" Value="{DynamicResource {x:Static vsshell:VsBrushes.WindowTextKey}}"/>
            <Setter Property="HorizontalContentAlignment" Value="Left"/>
            <Setter Property="Padding" Value="5,3,5,3"/>
            <Setter Property="Template">
                <Setter.Value>
                    <ControlTemplate TargetType="{x:Type GridViewColumnHeader}">
                        <Grid SnapsToDevicePixels="True">
                            <Border x:Name="HeaderBorder"
                                    BorderThickness="0,0,0,1"
                                    BorderBrush="{DynamicResource {x:Static vsshell:VsBrushes.ToolWindowBorderKey}}"
                                    Background="{TemplateBinding Background}">
                                <ContentPresenter Margin="{TemplateBinding Padding}"
                                                  VerticalAlignment="Center"
                                                  HorizontalAlignment="{TemplateBinding HorizontalContentAlignment}"
                                                  RecognizesAccessKey="True"/>
                            </Border>
                            <Thumb x:Name="PART_HeaderGripper" HorizontalAlignment="Right" Width="8" Cursor="SizeWE">
                                <Thumb.Style>
                                    <Style TargetType="{x:Type Thumb}">
                                        <Setter Property="Template">
                                            <Setter.Value>
                                                <ControlTemplate TargetType="{x:Type Thumb}">
                                                    <Border Background="Transparent"
                                                            BorderBrush="{DynamicResource {x:Static vsshell:VsBrushes.ToolWindowBorderKey}}"
                                                            BorderThickness="0,0,1,0"/>
                                                </ControlTemplate>
                                            </Setter.Value>
                                        </Setter>
                                    </Style>
                                </Thumb.Style>
                            </Thumb>
                        </Grid>
                        <ControlTemplate.Triggers>
                            <Trigger Property="IsMouseOver" Value="True">
                                <Setter TargetName="HeaderBorder" Property="Background"
                                        Value="{DynamicResource {x:Static vsshell:VsBrushes.ToolWindowTabMouseOverBackgroundBeginKey}}"/>
                            </Trigger>
                            <Trigger Property="IsPressed" Value="True">
                                <Setter TargetName="HeaderBorder" Property="Background"
                                        Value="{DynamicResource {x:Static vsshell:VsBrushes.CommandBarSelectedBorderKey}}"/>
                            </Trigger>
                            <Trigger Property="Role" Value="Padding">
                                <Setter TargetName="PART_HeaderGripper" Property="Visibility" Value="Collapsed"/>
                            </Trigger>
                        </ControlTemplate.Triggers>
                    </ControlTemplate>
                </Setter.Value>
            </Setter>
        </Style>
        <Style TargetType="Border">
            <Setter Property="BorderBrush" Value="{DynamicResource {x:Static vsshell:VsBrushes.ToolWindowBorderKey}}"/>
            <Setter Property="BorderThickness" Value="1"/>
        </Style>
        <Image x:Key="IconBookmarkAdd"    x:Shared="False" Source="/UnrealVS;component/Resources/BookmarkAdd.png"    Width="16" Height="16"/>
        <Image x:Key="IconBookmarkRemove" x:Shared="False" Source="/UnrealVS;component/Resources/BookmarkRemove.png" Width="16" Height="16"/>
        <Image x:Key="IconCopyInclude"    x:Shared="False" Source="/UnrealVS;component/Resources/CopyInclude.png"    Width="16" Height="16"/>
        <Image x:Key="IconInsertInclude"  x:Shared="False" Source="/UnrealVS;component/Resources/InsertInclude.png"  Width="16" Height="16"/>
    </UserControl.Resources>
    <Grid>
        <Grid.RowDefinitions>
            <RowDefinition Height="*"/>
        </Grid.RowDefinitions>
        <Grid>
            <Grid.RowDefinitions>
                <RowDefinition Height="auto"/>
                <RowDefinition Height="*"/>
                <RowDefinition Height="auto"/>
            </Grid.RowDefinitions>
            <Border Margin="13,16,13,0" Padding="5" BorderBrush="{DynamicResource {x:Static vsshell:VsBrushes.ToolWindowBorderKey}}">
                <Grid>
                    <Grid.ColumnDefinitions>
                        <ColumnDefinition Width="auto"/>
                        <ColumnDefinition Width="*"/>
                        <ColumnDefinition Width="auto"/>
                    </Grid.ColumnDefinitions>
                    <TextBlock Text="Filter:" Padding="3"/>
                    <TextBox Grid.Column="1" Name="FilterEditBox" CaretBrush="{DynamicResource {x:Static vsshell:VsBrushes.CommandBarTextActiveKey}}" Padding="3"/>
                    <Button Grid.Column="2" Name="RefreshButton" Click="RefreshButton_Click"
                            Width="22" Height="22" Margin="3,0,0,0" Padding="3"
                            ToolTip="Refresh (F5)"
                            Background="Transparent" BorderThickness="0"
                            VerticalAlignment="Center">
                        <Path Width="12" Height="12" Stretch="Uniform"
                              Data="M17.65 6.35C16.2 4.9 14.21 4 12 4c-4.42 0-7.99 3.58-7.99 8s3.57 8 7.99 8c3.73 0 6.84-2.55 7.73-6h-2.08c-.82 2.33-3.04 4-5.65 4-3.31 0-6-2.69-6-6s2.69-6 6-6c1.66 0 3.14.69 4.22 1.78L13 11h7V4l-2.35 2.35z"
                              Fill="{DynamicResource {x:Static vsshell:VsBrushes.WindowTextKey}}"/>
                    </Button>
                </Grid>
            </Border>
            <TabControl Name="FilesListTab" Grid.Row="1" IsTabStop="False" Background="Transparent" Margin="10" BorderBrush="Transparent">
                <TabItem Header="All files" Name="AllFilesTab" Focusable="False">
                    <ListView Name="AllFilesListBox" GridViewColumnHeader.Click="ColumnHeader_Click" PreviewMouseRightButtonDown="FileListView_PreviewMouseRightButtonDown">
                        <ListView.View>
                            <GridView>
                                <GridViewColumn Header="Name" Width="200" DisplayMemberBinding="{Binding Name}">
                                    <GridViewColumn.HeaderContainerStyle>
                                        <Style TargetType="GridViewColumnHeader" BasedOn="{StaticResource {x:Type GridViewColumnHeader}}">
                                            <Setter Property="Tag" Value="Name"/>
                                        </Style>
                                    </GridViewColumn.HeaderContainerStyle>
                                </GridViewColumn>
                                <GridViewColumn Header="Module" Width="130" DisplayMemberBinding="{Binding Module}">
                                    <GridViewColumn.HeaderContainerStyle>
                                        <Style TargetType="GridViewColumnHeader" BasedOn="{StaticResource {x:Type GridViewColumnHeader}}">
                                            <Setter Property="Tag" Value="Module"/>
                                        </Style>
                                    </GridViewColumn.HeaderContainerStyle>
                                </GridViewColumn>
                                <GridViewColumn Header="Path" Width="500" DisplayMemberBinding="{Binding File}">
                                    <GridViewColumn.HeaderContainerStyle>
                                        <Style TargetType="GridViewColumnHeader" BasedOn="{StaticResource {x:Type GridViewColumnHeader}}">
                                            <Setter Property="Tag" Value="File"/>
                                        </Style>
                                    </GridViewColumn.HeaderContainerStyle>
                                </GridViewColumn>
                            </GridView>
                        </ListView.View>
                    </ListView>
                </TabItem>
                <TabItem Header="Bookmarked files" Name="BookmarkedFilesTab" Focusable="False">
                    <ListView Name="BookmarkedFilesListBox" GridViewColumnHeader.Click="ColumnHeader_Click" PreviewMouseRightButtonDown="FileListView_PreviewMouseRightButtonDown">
                        <ListView.View>
                            <GridView>
                                <GridViewColumn Header="Name" Width="200" DisplayMemberBinding="{Binding Name}">
                                    <GridViewColumn.HeaderContainerStyle>
                                        <Style TargetType="GridViewColumnHeader" BasedOn="{StaticResource {x:Type GridViewColumnHeader}}">
                                            <Setter Property="Tag" Value="Name"/>
                                        </Style>
                                    </GridViewColumn.HeaderContainerStyle>
                                </GridViewColumn>
                                <GridViewColumn Header="Module" Width="130" DisplayMemberBinding="{Binding Module}">
                                    <GridViewColumn.HeaderContainerStyle>
                                        <Style TargetType="GridViewColumnHeader" BasedOn="{StaticResource {x:Type GridViewColumnHeader}}">
                                            <Setter Property="Tag" Value="Module"/>
                                        </Style>
                                    </GridViewColumn.HeaderContainerStyle>
                                </GridViewColumn>
                                <GridViewColumn Header="Path" Width="500" DisplayMemberBinding="{Binding File}">
                                    <GridViewColumn.HeaderContainerStyle>
                                        <Style TargetType="GridViewColumnHeader" BasedOn="{StaticResource {x:Type GridViewColumnHeader}}">
                                            <Setter Property="Tag" Value="File"/>
                                        </Style>
                                    </GridViewColumn.HeaderContainerStyle>
                                </GridViewColumn>
                            </GridView>
                        </ListView.View>
                    </ListView>
                </TabItem>
                <TabItem Header="Recent files" Name="RecentFilesTab" Focusable="False">
                    <ListView Name="RecentFilesListBox" GridViewColumnHeader.Click="ColumnHeader_Click" PreviewMouseRightButtonDown="FileListView_PreviewMouseRightButtonDown">
                        <ListView.View>
                            <GridView>
                                <GridViewColumn Header="Name" Width="200" DisplayMemberBinding="{Binding Name}">
                                    <GridViewColumn.HeaderContainerStyle>
                                        <Style TargetType="GridViewColumnHeader" BasedOn="{StaticResource {x:Type GridViewColumnHeader}}">
                                            <Setter Property="Tag" Value="Name"/>
                                        </Style>
                                    </GridViewColumn.HeaderContainerStyle>
                                </GridViewColumn>
                                <GridViewColumn Header="Module" Width="130" DisplayMemberBinding="{Binding Module}">
                                    <GridViewColumn.HeaderContainerStyle>
                                        <Style TargetType="GridViewColumnHeader" BasedOn="{StaticResource {x:Type GridViewColumnHeader}}">
                                            <Setter Property="Tag" Value="Module"/>
                                        </Style>
                                    </GridViewColumn.HeaderContainerStyle>
                                </GridViewColumn>
                                <GridViewColumn Header="Path" Width="500" DisplayMemberBinding="{Binding File}">
                                    <GridViewColumn.HeaderContainerStyle>
                                        <Style TargetType="GridViewColumnHeader" BasedOn="{StaticResource {x:Type GridViewColumnHeader}}">
                                            <Setter Property="Tag" Value="File"/>
                                        </Style>
                                    </GridViewColumn.HeaderContainerStyle>
                                </GridViewColumn>
                            </GridView>
                        </ListView.View>
                    </ListView>
                </TabItem>
            </TabControl>
            <Border Grid.Row="2" Margin="13,0,13,16" Padding="5" BorderBrush="{DynamicResource {x:Static vsshell:VsBrushes.ToolWindowBorderKey}}">
                <TextBox Name="StatusText" Text="" BorderThickness="0" IsReadOnly="True" IsTabStop="False"/>
            </Border>
        </Grid>
        <Border Name="HelpDialog" BorderThickness="4" Width="auto" Height="auto" Visibility="Collapsed" Background="{DynamicResource {x:Static vsshell:VsBrushes.EditorExpansionFillKey}}" HorizontalAlignment="Center" VerticalAlignment="Center">
            <StackPanel Margin="20">
                <TextBlock Text="Help (Press escape to close this popup)"/>
                <StackPanel Margin="6">
                    <TextBlock Text="A file browser specialized for Unreal solutions. The browser is aware of .build.cs files"/>
                    <TextBlock Text="To filter the file browser, simply type words, separated by space."/>
                </StackPanel>
                <TextBlock Text=""/>
                <TextBlock Text="Keys"/>
                <Grid>
                    <Grid.ColumnDefinitions>
                        <ColumnDefinition Width="auto"/>
                        <ColumnDefinition Width="auto"/>
                    </Grid.ColumnDefinitions>
                    <StackPanel Grid.Column="0" Margin="6">
                        <TextBlock Text="Escape"/>
                        <TextBlock Text="Enter"/>
                        <TextBlock Text="Up/Down"/>
                        <TextBlock Text="Left/right"/>
                        <TextBlock Text="Tab"/>
                        <TextBlock Text="Insert"/>
                        <TextBlock Text="Delete"/>
                        <TextBlock Text="F1"/>
                        <TextBlock Text="F5"/>
                        <TextBlock Text="Ctrl-I"/>
                        <TextBlock Text="Ctrl-Shift-I"/>
                    </StackPanel>
                    <StackPanel Grid.Column="1" Margin="6">
                        <TextBlock Text="- Close FileBrowser"/>
                        <TextBlock Text="- Open selected file"/>
                        <TextBlock Text="- Select file"/>
                        <TextBlock Text="- Move between all files list and bookmarked files list"/>
                        <TextBlock Text="- Toggle between filter edit box and file lists"/>
                        <TextBlock Text="- Insert bookmarked file"/>
                        <TextBlock Text="- Delete bookmarked file"/>
                        <TextBlock Text="- This help"/>
                        <TextBlock Text="- Refresh file lists with current visual studio solution"/>
                        <TextBlock Text="- Create a #include &quot;file&quot; statement to clipboard"/>
                        <TextBlock Text="- Create a #include &quot;file&quot; statement and paste it into active document"/>
                    </StackPanel>
                </Grid>
                <TextBlock Text=""/>
                <TextBlock Text="Bookmarks"/>
                <StackPanel Margin="6">
                    <TextBlock Text="Can be used as a way to store favourite files and quickly navigate."/>
                    <TextBlock Text="The list of bookmarks is solution specific and are stored next to the .sln file with extension .unrealvs"/>
                </StackPanel>
            </StackPanel>
        </Border>
        <Border Name="RefreshingDialog" Focusable="False" Width="300" Height="100" BorderThickness="4" Visibility="Collapsed" Background="{DynamicResource {x:Static vsshell:VsBrushes.EditorExpansionFillKey}}" HorizontalAlignment="Center" VerticalAlignment="Center">
            <TextBlock Name="RefreshingText" Focusable="False" Text="" Width="auto" Height="auto" HorizontalAlignment="Center" VerticalAlignment="Center"/>
        </Border>
    </Grid>
</UserControl>
