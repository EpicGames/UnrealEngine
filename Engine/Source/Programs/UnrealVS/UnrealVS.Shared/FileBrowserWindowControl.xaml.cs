// Copyright Epic Games, Inc. All Rights Reserved.

using EnvDTE;
using Microsoft.VisualStudio.Shell;
using Microsoft.VisualStudio.Shell.Interop;
using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text;
using System.Text.Json;
using System.ComponentModel;
using System.Windows;
using System.Windows.Controls;
using System.Windows.Data;
using System.Windows.Input;
using System.Windows.Media;
using System.Windows.Threading;
using System.Windows.Media.Imaging;

namespace UnrealVS
{
	[global::System.Diagnostics.CodeAnalysis.SuppressMessageAttribute("", "VSTHRD010")]
	public partial class FileBrowserWindowControl : UserControl
	{
		private FileBrowserWindow Window;
		private bool bClearFilter;
		private bool bIsRefreshing;
		private ListBox ActiveFilesListBox;

		public class FileItem
		{
			public string Name { get; set; }
			public string File { get; set; }
			public string Module { get; set; }
		}

		private string UnrealVsFileName;

		private FileItem[] AllFileItems;
		private List<FileItem> BookmarkedFileItems;
		private List<FileItem> RecentFileItems;

		private Dictionary<string, FileItem> AllFileItemsLookup;

		private Dictionary<ListBox, (GridViewColumnHeader Header, string OriginalContent, ListSortDirection Direction)> ColumnSortState;

		public FileBrowserWindowControl(FileBrowserWindow InWindow)
		{
			Window = InWindow;
			InitializeComponent();

			AllFileItems = Array.Empty<FileItem>();
			BookmarkedFileItems = new List<FileItem>();
			RecentFileItems = new List<FileItem>();
			ColumnSortState = new Dictionary<ListBox, (GridViewColumnHeader, string, ListSortDirection)>();
			IsVisibleChanged += FileBrowserWindowControl_IsVisibleChanged;
			foreach (ListBox FileListBox in new[] { AllFilesListBox, BookmarkedFilesListBox, RecentFilesListBox })
			{
				FileListBox.ItemsSource = new List<FileItem>();
				FileListBox.KeyDown += FileListView_KeyDown;
				FileListBox.PreviewKeyDown += FileListView_PreviewKeyDown;
				FileListBox.SelectionChanged += FileListView_SelectionChanged;
				FileListBox.GotFocus += FileListView_GotFocus;
				FileListBox.MouseDoubleClick += ListView_MouseDoubleClick;
				FileListBox.SelectionMode = SelectionMode.Extended;
			}
			FilterEditBox.TextChanged += FilterEditBox_TextChanged;
			FilterEditBox.KeyDown += FilterEditBox_KeyDown;
			FilterEditBox.PreviewKeyDown += FilterEditBox_PreviewKeyDown;
			DataObject.AddPastingHandler(FilterEditBox, FilterEditBox_OnPaste);
			FilesListTab.SelectionChanged += FilesListTab_SelectionChanged;
			FilesListTab.SelectedIndex = 0;

			ActiveFilesListBox = AllFilesListBox;
			PreviewKeyDown += FileBrowserWindowControl_PreviewKeyDown;
		}

		private void FilterEditBox_OnPaste(object Sender, DataObjectPastingEventArgs E)
		{
			bool bIsText = E.SourceDataObject.GetDataPresent(DataFormats.UnicodeText, true);
			if (!bIsText)
			{
				return;
			}

			string Text = E.SourceDataObject.GetData(DataFormats.UnicodeText) as string;

			if (string.IsNullOrEmpty(Text) || Text.IndexOfAny(Path.GetInvalidPathChars()) != -1)
			{
				return;
			}

			FilterEditBox.Text = Path.GetFileName(Text);
			E.CancelCommand();
			FilterEditBox.CaretIndex = FilterEditBox.Text.Length;
		}

		[STAThread]
		private void FileBrowserWindowControl_PreviewKeyDown(object Sender, KeyEventArgs E)
		{
			if (FilterEditBox.IsKeyboardFocused)
			{
				return;
			}

			if (Keyboard.Modifiers.HasFlag(ModifierKeys.Control) && E.Key == Key.V || Keyboard.Modifiers.HasFlag(ModifierKeys.Shift) && E.Key == Key.Insert)
			{
				string Text = Clipboard.GetText();

				if (string.IsNullOrEmpty(Text) || Text.IndexOfAny(Path.GetInvalidPathChars()) != -1)
				{
					return;
				}

				FilterEditBox.Text = Path.GetFileName(Text);
				FilterEditBox.SelectAll();
			}
		}

		private void FilesListTab_SelectionChanged(object Sender, SelectionChangedEventArgs E)
		{
			switch (FilesListTab.SelectedIndex)
			{
				case 0:
					if (ActiveFilesListBox == AllFilesListBox)
					{
						return;
					}
					ActiveFilesListBox = AllFilesListBox;
					break;
				case 1:
					if (ActiveFilesListBox == BookmarkedFilesListBox)
					{
						return;
					}
					ActiveFilesListBox = BookmarkedFilesListBox;
					break;
				case 2:
					if (ActiveFilesListBox == RecentFilesListBox)
					{
						return;
					}
					ActiveFilesListBox = RecentFilesListBox;
					break;
			}
			FilesListTab.UpdateLayout();
			RefreshStatusBox();
			AsyncFocusSelectedItem();
		}

		private void RefreshButton_Click(object Sender, RoutedEventArgs E)
		{
			AsyncRefreshFileList();
		}

		// ── Context menu ─────────────────────────────────────────────────────────

		private void FileListView_PreviewMouseRightButtonDown(object Sender, MouseButtonEventArgs E)
		{
			// Walk up the visual tree to find the ListViewItem under the cursor.
			DependencyObject Obj = E.OriginalSource as DependencyObject;
			while (Obj != null && !(Obj is ListViewItem))
			{
				Obj = VisualTreeHelper.GetParent(Obj);
			}
			if (!(Obj is ListViewItem))
			{
				return;
			}

			ListBox ListBox = (ListBox)Sender;
			FileItem Item = ((ListViewItem)Obj).DataContext as FileItem;
			if (Item == null || Item.Name == "")
			{
				return;
			}

			ListBox.SelectedItem = Item;
			ListBox.Focus();

			bool bIsBookmarked = BookmarkedFileItems.Contains(Item);
			string IncludePath = GetIncludePath(Item);
			bool bHasActiveDoc = UnrealVSPackage.Instance.DTE?.ActiveDocument != null;

			ContextMenu Menu = new ContextMenu();

			// 1. Bookmark toggle
			MenuItem BookmarkItem = new MenuItem();
			if (bIsBookmarked)
			{
				BookmarkItem.Header = "Remove from Bookmarks";
				BookmarkItem.Icon = FindResource("IconBookmarkRemove");
				BookmarkItem.ToolTip = "Remove this file from the bookmarked list (Delete)";
			}
			else
			{
				BookmarkItem.Header = "Add to Bookmarks";
				BookmarkItem.Icon = FindResource("IconBookmarkAdd");
				BookmarkItem.ToolTip = "Add this file to the bookmarked list (Insert)";
			}
			BookmarkItem.Tag = Item;
			BookmarkItem.Click += FileItemContextMenu_ToggleBookmark_Click;
			Menu.Items.Add(BookmarkItem);

			Menu.Items.Add(new Separator());

			// 2. Copy #include to clipboard
			MenuItem CopyItem = new MenuItem();
			CopyItem.Header = "Copy #include to Clipboard";
			CopyItem.Icon = FindResource("IconCopyInclude");
			CopyItem.ToolTip = "Copy a #include \"...\" statement to the clipboard (Ctrl+I)";
			CopyItem.IsEnabled = IncludePath != null;
			CopyItem.Tag = Item;
			CopyItem.Click += FileItemContextMenu_CopyInclude_Click;
			Menu.Items.Add(CopyItem);

			// 3. Insert #include into active document
			MenuItem InsertItem = new MenuItem();
			InsertItem.Header = "Insert #include into Active Document";
			InsertItem.Icon = FindResource("IconInsertInclude");
			InsertItem.ToolTip = "Insert a #include \"...\" statement into the active document at the correct position (Ctrl+Shift+I)";
			InsertItem.IsEnabled = IncludePath != null && bHasActiveDoc;
			InsertItem.Tag = Item;
			InsertItem.Click += FileItemContextMenu_InsertInclude_Click;
			Menu.Items.Add(InsertItem);

			Menu.PlacementTarget = (UIElement)Sender;
			Menu.IsOpen = true;
			E.Handled = true;
		}

		private void FileItemContextMenu_ToggleBookmark_Click(object Sender, RoutedEventArgs E)
		{
			FileItem Item = (FileItem)((MenuItem)Sender).Tag;
			if (BookmarkedFileItems.Contains(Item))
			{
				BookmarkedFileItems.Remove(Item);
			}
			else
			{
				BookmarkedFileItems.Add(Item);
			}
			RefreshListView(BookmarkedFilesListBox, BookmarkedFileItems.ToArray());
			SaveSolutionSettings();
		}

		private void FileItemContextMenu_CopyInclude_Click(object Sender, RoutedEventArgs E)
		{
			FileItem Item = (FileItem)((MenuItem)Sender).Tag;
			string IncludePath = GetIncludePath(Item);
			if (IncludePath == null)
			{
				return;
			}
			Clipboard.SetText($"#include \"{IncludePath}\"");
		}

		private void FileItemContextMenu_InsertInclude_Click(object Sender, RoutedEventArgs E)
		{
			FileItem Item = (FileItem)((MenuItem)Sender).Tag;
			string IncludePath = GetIncludePath(Item);
			if (IncludePath == null)
			{
				return;
			}
			Clipboard.SetText($"#include \"{IncludePath}\"");
			InsertIncludeIntoDocument(Item);
		}

		// ── InsertIncludeIntoDocument ──────

		private void InsertIncludeIntoDocument(FileItem Item)
		{
			if (Item == null)
			{
				return;
			}
			Document ActiveDoc = UnrealVSPackage.Instance.DTE.ActiveDocument;
			if (ActiveDoc == null)
			{
				return;
			}
			TextDocument Doc = (TextDocument)(ActiveDoc.Object("TextDocument"));
			if (Doc == null)
			{
				return;
			}

			bool bAddSpace = false;
			string IncludePath = GetIncludePath(Item);
			string PasteString = $"#include \"{IncludePath}\"";

			bool bSkipFirstInclude = ActiveDoc.FullName.EndsWith(".cpp");
			int AddBeforeLine = -1;
			int LastIncludeLine = -1;

			bool bIsInComment = false;
			EditPoint EditPoint = Doc.StartPoint.CreateEditPoint();
			int LastLine = Doc.EndPoint.Line;
			for (int LineIndex = 1; LineIndex < LastLine; ++LineIndex)
			{
				string Str = EditPoint.GetLines(LineIndex, LineIndex + 1).Trim();
				if (Str.Length == 0)
				{
					continue;
				}

				if (bIsInComment)
				{
					if (Str.IndexOf("*/") != -1)
					{
						bIsInComment = false;
					}
					continue;
				}

				if (Str.StartsWith("//"))
				{
					continue;
				}

				if (Str.StartsWith("/*"))
				{
					bIsInComment = true;
					continue;
				}

				if (Str.StartsWith("#include"))
				{
					if (Str == PasteString)
					{
						Doc.Selection.MoveTo(LineIndex, 0);
						Doc.Selection.SelectLine();
						EditPoint.TryToShow();
						return;
					}

					bool bIsFirst = LastIncludeLine == -1;
					int PrevLastIncludeLine = LastIncludeLine;
					LastIncludeLine = LineIndex;
					if (bSkipFirstInclude && bIsFirst)
					{
						continue;
					}

					ReadOnlySpan<char> Span = Str.AsSpan(9);
					int FirstQuote = Span.IndexOf('"');
					if (FirstQuote != -1)
					{
						Span = Span.Slice(FirstQuote + 1);
						int SecondQuote = Span.IndexOf('"');
						if (SecondQuote != -1)
						{
							Span = Span.Slice(0, SecondQuote).Trim();
							bool bIsGeneratedInclude = Span.IndexOf(".generated.".AsSpan()) != -1;
							if (IncludePath.AsSpan().CompareTo(Span, StringComparison.Ordinal) < 0 || bIsGeneratedInclude)
							{
								if (bIsGeneratedInclude && PrevLastIncludeLine != -1)
								{
									LastIncludeLine = PrevLastIncludeLine;
								}
								else
								{
									AddBeforeLine = LineIndex;
								}
								break;
							}
						}
					}
					else if (Span.IndexOf("UE_INLINE_GENERATED_CPP".AsSpan()) != -1)
					{
						LastIncludeLine = PrevLastIncludeLine;
						break;
					}
					continue;
				}

				if (Str.StartsWith("#pragma once"))
				{
					LastIncludeLine = LineIndex + 1;
					continue;
				}

				LastIncludeLine = LineIndex - 1;
				bAddSpace = true;
				break;  // Unknown code
			}

			if (AddBeforeLine == -1 && LastIncludeLine != -1)
			{
				AddBeforeLine = LastIncludeLine + 1;
			}

			if (AddBeforeLine == -1)
			{
				AddBeforeLine = LastLine;
			}

			EditPoint.LineDown(AddBeforeLine - 1);
			EditPoint.Insert($"{PasteString}\r\n");
			if (bAddSpace)
			{
				EditPoint.Insert("\r\n");
			}
			Doc.Selection.MoveTo(AddBeforeLine, 0);
			Doc.Selection.SelectLine();
			EditPoint.TryToShow();
		}

		private void FilterEditBox_KeyDown(object Sender, KeyEventArgs E)
		{
			if (E.Key == Key.Tab)
			{
				E.Handled = true;
				FocusListViewItem(ActiveFilesListBox, ActiveFilesListBox.SelectedIndex);
				return;
			}
		}

		private void FilterEditBox_PreviewKeyDown(object Sender, KeyEventArgs E)
		{
			ListBox TargetListBox = ActiveFilesListBox;
			Func<FileItem> GetSelectedItem = new Func<FileItem>(() =>
			{
				if (TargetListBox.SelectedItems.Count == 0)
					return (FileItem)TargetListBox.Items[0];
				else
					return (FileItem)TargetListBox.SelectedItems[TargetListBox.SelectedItems.Count - 1];
			});

			Action<int> MoveFunc = new Action<int>((int Steps) =>
			{
				FileItem SelectedItem = GetSelectedItem();
				int Index = TargetListBox.Items.IndexOf(SelectedItem);
				int NewIndex = Math.Min(Math.Max(0, Index + Steps), TargetListBox.Items.Count - 1);
				if (NewIndex == Index)
				{
					return;
				}
				SelectedItem = (FileItem)TargetListBox.Items[NewIndex];
				if (SelectedItem == null)
				{
					return;
				}
				TargetListBox.SelectedItems.Clear();
				TargetListBox.SelectedItems.Add(SelectedItem);
				TargetListBox.ScrollIntoView(SelectedItem);
			});

			if (E.Key == Key.Down)
			{
				MoveFunc(1);
				return;
			}
			if (E.Key == Key.Up)
			{
				MoveFunc(-1);
				return;
			}
			if (E.Key == Key.Enter)
			{
				if (OpenSelectedFile())
				{
					HidePanelAfterSelection();
				}
				return;
			}
			if (E.Key == Key.PageDown)
			{
				E.Handled = true;
				ListBoxItem LastVisible = GetVisibleListViewElement(TargetListBox, (int)TargetListBox.ActualHeight - 22);
				if (LastVisible == null)
				{
					return;
				}
				object LastItem = TargetListBox.ItemContainerGenerator.ItemFromContainer(LastVisible);
				FileItem SelectedItem = GetSelectedItem();
				if (LastItem != SelectedItem)
				{
					TargetListBox.SelectedItems.Clear();
					TargetListBox.SelectedItems.Add(LastItem);
					TargetListBox.ScrollIntoView(LastItem);
				}
				else
				{
					MoveFunc(((int)(TargetListBox.ActualHeight / LastVisible.ActualHeight)) - 1);
				}
				return;
			}
			if (E.Key == Key.PageUp)
			{
				E.Handled = true;
				ListBoxItem FirstVisible = GetVisibleListViewElement(TargetListBox, 10);
				if (FirstVisible == null)
				{
					return;
				}
				object FirstItem = TargetListBox.ItemContainerGenerator.ItemFromContainer(FirstVisible);
				FileItem SelectedItem = GetSelectedItem();
				if (FirstItem != SelectedItem)
				{
					TargetListBox.SelectedItems.Clear();
					TargetListBox.SelectedItems.Add(FirstItem);
					TargetListBox.ScrollIntoView(FirstItem);
				}
				else
				{
					MoveFunc(1 - (int)(TargetListBox.ActualHeight / FirstVisible.ActualHeight));
				}
			}
		}

		private void FilterEditBox_TextChanged(object Sender, TextChangedEventArgs E)
		{
			RefreshListViews();
			//AllFilesListBox.SelectedIndex = 0;
		}

		private void FileListView_GotFocus(object Sender, RoutedEventArgs E)
		{
			ActiveFilesListBox = (ListBox)E.Source;
		}

		private void FileListView_SelectionChanged(object Sender, SelectionChangedEventArgs E)
		{
			RefreshStatusBox();
		}

		private void FileListView_KeyDown(object Sender, KeyEventArgs E)
		{
			if ((Keyboard.Modifiers & ModifierKeys.Control) != 0)
			{
				if (E.Key == Key.I)
				{
					if (!CreateIncludePath())
					{
						return;
					}

					HidePanel();

					if ((Keyboard.Modifiers & ModifierKeys.Shift) == 0)
					{
						return;
					}

					InsertIncludeIntoDocument((FileItem)(ActiveFilesListBox.SelectedItems[0]));
				}
				return;
			}
			if (E.Key == Key.Left)
			{
				if (ActiveFilesListBox != AllFilesListBox)
				{
					--FilesListTab.SelectedIndex;
				}
				//if (ActiveFilesListBox == BookmarkedFilesListBox)
				//    ToggleListView(BookmarkedFilesListBox, AllFilesListBox);
				return;
			}

			if (E.Key == Key.Right)
			{
				if (ActiveFilesListBox != RecentFilesListBox)// && ((List<FileItem>)BookmarkedFilesListBox.ItemsSource).Count > 0)
				{
					++FilesListTab.SelectedIndex;
				}
				//if (ActiveFilesListBox == AllFilesListBox)// && ((List<FileItem>)BookmarkedFilesListBox.ItemsSource).Count > 0)
				//   FilesListTab.SelectedIndex = 1;
				//if (ActiveFilesListBox == AllFilesListBox && ((List<FileItem>)BookmarkedFilesListBox.ItemsSource).Count > 0)
				//     ToggleListView(AllFilesListBox, BookmarkedFilesListBox);
				return;
			}

			if (E.Key == Key.Tab)
			{
				E.Handled = true;
				FilterEditBox.Focus();
				return;
			}

			if (E.Key == Key.Insert)
			{
				if (ActiveFilesListBox != AllFilesListBox)
				{
					return;
				}
				bool bBookmarksModified = false;
				foreach (object SelectedItem in ActiveFilesListBox.SelectedItems)
				{
					FileItem Item = (FileItem)SelectedItem;
					if (Item == null || Item.Name == "")
					{
						continue;
					}
					if (BookmarkedFileItems.Contains(Item))
					{
						continue;
					}
					BookmarkedFileItems.Add(Item);
					bBookmarksModified = true;
				}
				if (!bBookmarksModified)
				{
					return;
				}
				RefreshListView(BookmarkedFilesListBox, BookmarkedFileItems.ToArray());
				//if (AllFilesListBox.SelectedIndex < AllFilesListBox.Items.Count - 1)
				//    FocusListViewItem(AllFilesListBox, AllFilesListBox.SelectedIndex + 1);
				SaveSolutionSettings();
				return;
			}

			if (E.Key == Key.Delete)
			{
				if (ActiveFilesListBox != BookmarkedFilesListBox)
				{
					return;
				}
				bool bBookmarksModified = false;

				int Index = BookmarkedFilesListBox.SelectedIndex;
				foreach (object SelectedItem in ActiveFilesListBox.SelectedItems)
				{
					FileItem Item = (FileItem)SelectedItem;
					if (Item == null || Item.Name == "")
					{
						continue;
					}
					if (!BookmarkedFileItems.Remove(Item))
					{
						continue;
					}
					bBookmarksModified = true;
				}
				if (bBookmarksModified)
				{
					RefreshListView(BookmarkedFilesListBox, BookmarkedFileItems.ToArray());
					if (Index == BookmarkedFilesListBox.Items.Count)
					{
						if (Index == 0)
						{
							FilesListTab.SelectedIndex = 0;//ActiveFilesListBox = AllFilesListBox;
						}
						else
						{
							Index -= 1;
						}
					}
					BookmarkedFilesListBox.SelectedIndex = Index;
					AsyncFocusSelectedItem();
					SaveSolutionSettings();
				}
				return;
			}

			if (E.Key == Key.Enter)
			{
				if (OpenSelectedFile())
				{
					HidePanelAfterSelection();
				}
				return;
			}

			bool bResetSelection = false;

			if (E.Key == Key.Back)
			{
				string Str = FilterEditBox.Text;
				if (string.IsNullOrEmpty(Str))
					return;
				if (bClearFilter)
				{
					bResetSelection = true;
					bClearFilter = false;
					FilterEditBox.Text = "";
				}
				else
				{
					FilterEditBox.Text = Str.Substring(0, Str.Length - 1);
				}
			}
			else
			{
				char Ch = GetCharFromKey(E.Key);
				if (Ch == 0)
					return;
				if (bClearFilter)
				{
					bResetSelection = true;
					bClearFilter = false;
					FilterEditBox.Text = "";
				}
				FilterEditBox.Text += Ch;
				FilterEditBox.SelectAll();
			}

			if (bResetSelection)
			{
				AllFilesListBox.SelectedIndex = 0;
				BookmarkedFilesListBox.SelectedIndex = 0;
				RecentFilesListBox.SelectedIndex = 0;
			}

			AsyncFocusSelectedItem();

			E.Handled = true;
		}

		private void ListView_MouseDoubleClick(object Sender, MouseButtonEventArgs E)
		{
			if (OpenSelectedFile())
			{
				HidePanelAfterSelection();
			}
		}

		private bool OpenSelectedFile()
		{
			List<string> ToOpen = new List<string>();
			foreach (object SelectedItem in ActiveFilesListBox.SelectedItems)
			{
				FileItem Item = (FileItem)SelectedItem;
				if (Item == null || Item.Name == "")
				{
					continue;
				}
				ToOpen.Add(Item.File);
			}

			ParseLineNumber(FilterEditBox.Text, out int LineNumber);

			ThreadHelper.JoinableTaskFactory.Run(async delegate
			{
				await ThreadHelper.JoinableTaskFactory.SwitchToMainThreadAsync();
				foreach (string F in ToOpen)
				{
					VsShellUtilities.OpenDocument(UnrealVSPackage.Instance, F);
					if (LineNumber > 0)
					{
						Document ActiveDoc = UnrealVSPackage.Instance.DTE.ActiveDocument;
						TextDocument TextDoc = ActiveDoc?.Object("TextDocument") as TextDocument;
						if (TextDoc != null)
						{
							TextDoc.Selection.GotoLine(LineNumber, true);
						}
					}
				}
			});

			return ToOpen.Count != 0;
		}

		private string GetIncludePath(FileItem Item)
		{
			if (Item == null || Item.Name == "")
			{
				return null;
			}

			if (CopyIncludePath.BuildIncludePath(Item.File, out string IncludePath))
			{
				return IncludePath;
			}
			return null;
		}

		private bool CreateIncludePath()
		{
			string IncludeStrings = "";
			foreach (object SelectedItem in ActiveFilesListBox.SelectedItems)
			{
				string IncludePath = GetIncludePath((FileItem)SelectedItem);
				if (IncludePath != null)
				{
					IncludeStrings += $"#include \"{IncludePath}\"\r\n";
				}
			}
			if (IncludeStrings == "")
				return false;
			Clipboard.SetText(IncludeStrings);
			return true;
		}

		private void ToggleListView(ListBox From, ListBox To)
		{
			ListBoxItem FirstVisibleFrom = GetVisibleListViewElement(From, 10);
			if (FirstVisibleFrom == null)
			{
				return;
			}
			int FirstVisibleFromIndex = From.ItemContainerGenerator.IndexFromContainer(FirstVisibleFrom);
			ListBoxItem FirstVisibleTo = GetVisibleListViewElement(To, 10);
			if (FirstVisibleTo == null)
			{
				return;
			}
			int FirstVisibleToIndex = To.ItemContainerGenerator.IndexFromContainer(FirstVisibleTo);
			int ToSelectIndex = FirstVisibleToIndex + From.SelectedIndex - FirstVisibleFromIndex;
			if (ToSelectIndex >= To.Items.Count)
				ToSelectIndex = To.Items.Count - 1;
			FocusListViewItem(To, ToSelectIndex);
		}

		private void FocusListViewItem(ListBox ListView, int ItemIndex)
		{
			if (ItemIndex == -1)
			{
				ItemIndex = 0;
			}
			ListView.SelectedIndex = ItemIndex;
			if (ListView.Items.Count > 0)
			{
				(ListView.ItemContainerGenerator.ContainerFromIndex(ItemIndex) as ListBoxItem)?.Focus();
			}
			else
			{
				ListView.Focus();
			}
			RefreshStatusBox();
		}

		private void AsyncFocusSelectedItem()
		{
			double Interval = 0.1;
			DispatcherTimer Timer = new DispatcherTimer(DispatcherPriority.Normal);
			Timer.Tick += (S, E) =>
			{
				List<FileItem> Items = (List<FileItem>)ActiveFilesListBox.ItemsSource;
				if (Items.Count == 0)
				{
					ActiveFilesListBox.Focus();
					Timer.Stop();
					return;
				}
				ListBoxItem Item = (ActiveFilesListBox.ItemContainerGenerator.ContainerFromIndex(Math.Max(0, ActiveFilesListBox.SelectedIndex)) as ListBoxItem);
				if (Item == null)
				{
					Timer.Interval = TimeSpan.FromSeconds(Interval);
					Interval = Math.Min(Interval + 0.1, 1.0);
					return;
				}
				Item.Focus();
				Timer.Stop();
			};
			Timer.Start();
		}

		private void RefreshListViews()
		{
			RefreshListView(AllFilesListBox, AllFileItems);
			RefreshListView(BookmarkedFilesListBox, BookmarkedFileItems.ToArray());
			RefreshListView(RecentFilesListBox, RecentFileItems.ToArray());
		}

		private struct Score
		{
			public int Sorting;
			public int Index;
		}

		// Parses an optional line number suffix from a filter string.
		// Supports "name:123" and "name(123)" syntax.
		// Returns the filter text with the line number portion removed, and sets LineNumber (-1 if absent).
		private static string ParseLineNumber(string FilterText, out int LineNumber)
		{
			LineNumber = -1;
			string Trimmed = FilterText.TrimEnd();

			// "name(123)" pattern
			if (Trimmed.EndsWith(")"))
			{
				int OpenParen = Trimmed.LastIndexOf('(');
				if (OpenParen != -1)
				{
					string Inside = Trimmed.Substring(OpenParen + 1, Trimmed.Length - OpenParen - 2).Trim();
					if (Inside.Length > 0 && int.TryParse(Inside, out int N) && N > 0)
					{
						LineNumber = N;
						return Trimmed.Substring(0, OpenParen);
					}
				}
			}

			// "name:123" pattern
			int ColonIndex = Trimmed.LastIndexOf(':');
			if (ColonIndex != -1)
			{
				string After = Trimmed.Substring(ColonIndex + 1).Trim();
				if (After.Length > 0 && int.TryParse(After, out int N) && N > 0)
				{
					LineNumber = N;
					return Trimmed.Substring(0, ColonIndex);
				}
			}

			return FilterText;
		}

		private void RefreshListView(ListBox ListView, FileItem[] Source)
		{
			List<FileItem> FileItems = (List<FileItem>)ListView.ItemsSource;
			FileItems.Clear();

			string FilterStr = ParseLineNumber(FilterEditBox.Text, out int _LineNumber);
			// Strip line-number delimiter characters so they don't interfere with name matching
			FilterStr = FilterStr.Replace(':', ' ').Replace('(', ' ').Replace(')', ' ');
			string[] Filter = FilterStr.Split(new char[] { ' ' }, StringSplitOptions.RemoveEmptyEntries);

			HashSet<FileItem> Selections = new HashSet<FileItem>();
			foreach (object SelectedItem in ListView.SelectedItems)
			{
				FileItem Item = (FileItem)SelectedItem;
				if (Item != null && Item.Name != "")
					Selections.Add(Item);
			}

			int FilterLen = Filter.Length;
			if (FilterLen > 0)
			{
				List<ValueTuple<Score, FileItem>> FilterSortedItems = new List<ValueTuple<Score, FileItem>>();
				ValueTuple<int, int>[] ScoreArray = new ValueTuple<int, int>[5];
				int ItemIndex = 0;
				foreach (FileItem Item in Source)
				{
					int ScoreArrayIndex = 0;
					for (int i = 0; i != FilterLen; ++i)
					{
						string F = Filter[i];
						int Index = Item.Name.IndexOf(F, System.StringComparison.OrdinalIgnoreCase);
						if (Index == -1)
						{
							ScoreArrayIndex = 0;
							break;
						}
						ScoreArray[ScoreArrayIndex].Item1 = Index;
						ScoreArray[ScoreArrayIndex].Item2 = ScoreArrayIndex;
						++ScoreArrayIndex;
						if (ScoreArrayIndex == 5)
							break;
					}

					if (ScoreArrayIndex == 0)
						continue;

					// We want to sort the ones that match the filter orders first and prioritize the ones that starts with the first filter entry
					int SortingScore = 0;
					Array.Sort(ScoreArray, 0, ScoreArrayIndex);
					for (int i = 0; i != ScoreArrayIndex; ++i)
					{
						ValueTuple<int, int> ScoreEntry = ScoreArray[i];
						if (i == 0 && ScoreEntry.Item1 == 0 && ScoreEntry.Item2 == 0)
							SortingScore -= 100;
						SortingScore += (10 >> ScoreEntry.Item2) * i + ScoreEntry.Item1;
					}
					Score NewScore = new Score() { Sorting = SortingScore, Index = ItemIndex++ };
					FilterSortedItems.Add(new ValueTuple<Score, FileItem>(NewScore, Item));
				}

				FilterSortedItems.Sort((A, B) =>
				{
					if (A.Item1.Sorting != B.Item1.Sorting)
						return A.Item1.Sorting - B.Item1.Sorting;
					return A.Item1.Index - B.Item1.Index;
				});
				foreach (ValueTuple<Score, FileItem> SortedItem in FilterSortedItems)
					FileItems.Add(SortedItem.Item2);
			}
			else
			{
				FileItems.AddRange(Source);
			}

			int FileCount = FileItems.Count;

			// If empty we need to add one entry to make sure listview can be focused in windows
			if (FileItems.Count == 0)
				FileItems.Add(new FileItem() { Name = "" });

			CollectionViewSource.GetDefaultView(FileItems)?.Refresh();

			if (Selections.Count != 0)
			{
				foreach (FileItem Item in FileItems)
				{
					if (!Selections.Contains(Item))
						continue;
					ListView.SelectedItems.Add(Item);
					ListView.ScrollIntoView(Item);
				}
			}

			if (ListView.SelectedItems.Count == 0)
				ListView.SelectedItems.Add(FileItems[0]);


			if (ListView == AllFilesListBox)
				AllFilesTab.Header = $"All Files ({FileCount}/{Source.Length})";
			else if (ListView == BookmarkedFilesListBox)
				BookmarkedFilesTab.Header = $"Bookmarked Files ({FileCount}/{Source.Length})";
			else
				RecentFilesTab.Header = $"Recent Files ({FileCount}/{Source.Length})";
		}

		private void RefreshStatusBox()
		{
			System.Collections.IList Items = ActiveFilesListBox.SelectedItems;
			if (Items.Count == 0)
				StatusText.Text = "";
			else if (Items.Count == 1)
			{
				FileItem Item = (FileItem)Items[0];
				StatusText.Text = Item.File;
			}
			else
				StatusText.Text = "Multiple files selected";
		}

		private void FileListView_PreviewKeyDown(object Sender, KeyEventArgs E)
		{
			if (E.Key != Key.Space)
				return;
			if (bClearFilter)
			{
				bClearFilter = false;
				return;
			}
			FilterEditBox.Text += ' ';
			E.Handled = true;
		}

		internal void HandleEscape()
		{
			if (HelpDialog.Visibility == Visibility.Visible)
				HelpDialog.Visibility = Visibility.Collapsed;
			else
				HidePanel();
		}

		internal void HandleF1()
		{
			HelpDialog.Visibility = Visibility.Visible;
		}

		internal void HandleF5()
		{
			AsyncRefreshFileList();
		}

		internal void HandleSolutionChanged()
		{
			SaveSolutionSettings();
			if (IsVisible)
			{
				AsyncRefreshFileList();
			}
			else
			{
				AllFileItems = Array.Empty<FileItem>(); // Will trigger AsyncRefreshFileList when visible
			}
		}

		internal void HandleDocumentActivated(Document Document)
		{
			string FileName = Document.FullName;
			FileItem FileItem;
			if (AllFileItemsLookup == null || !AllFileItemsLookup.TryGetValue(FileName, out FileItem))
			{
				return;
			}

			RecentFileItems.RemoveAll((Item) => Item == FileItem);
			RecentFileItems.Insert(0, FileItem);

			int RecentMaxCount = 100;
			if (RecentFileItems.Count > RecentMaxCount)
			{
				RecentFileItems.RemoveRange(RecentMaxCount, RecentFileItems.Count - RecentMaxCount);
			}

			RefreshListView(RecentFilesListBox, RecentFileItems.ToArray());
		}

		private void AsyncRefreshFileList()
		{
			if (bIsRefreshing)
				return;
			bIsRefreshing = true;
			RefreshingText.Text = "Refreshing File Lists (0)";
			RefreshingDialog.Visibility = Visibility.Visible;

			DispatcherTimer Timer = new DispatcherTimer(DispatcherPriority.Render);
			Timer.Interval = TimeSpan.FromSeconds(0.2);
			SolutionTraverser Traverser = new SolutionTraverser();
			Timer.Tick += (S, E) =>
			{
				FileItem[] ListItems = Traverser.Update();
				if (ListItems == null)
				{
					RefreshingText.Text = $"Refreshing File Lists ({Traverser.HandledItemCount})";
					Timer.Interval = TimeSpan.FromSeconds(0.01);
					return;
				}
				AllFileItems = ListItems;

				Dictionary<string, FileItem> Lookup = new Dictionary<string, FileItem>();
				foreach (FileItem Item in AllFileItems)
				{
					Lookup[Item.File] = Item;
				}
				AllFileItemsLookup = Lookup;

				LoadSolutionSettings();

				//bClearFilter = true;
				RefreshListViews();

				//AllFilesListBox.SelectedIndex = 0;
				//BookmarkedFilesListBox.SelectedIndex = 0;
				//RecentFilesListBox.SelectedIndex = 0;
				AsyncFocusSelectedItem();

				Timer.Stop();
				RefreshingDialog.Visibility = Visibility.Collapsed;
				bIsRefreshing = false;
			};
			Timer.Start();
		}

		internal void HandleShow()
		{
			FilterEditBox.SelectAll();
			FilterEditBox.Focus();
			bClearFilter = true;
		}

		private void FileBrowserWindowControl_IsVisibleChanged(object Sender, DependencyPropertyChangedEventArgs E)
		{
			if (!(bool)E.NewValue)
			{
				return;
			}
			HandleShow();
			//FilesListTab.SelectedIndex = 0;

			if (AllFileItems.Length == 0)
			{
				RefreshListViews(); // To make sure we get one invisible entry
				AsyncRefreshFileList();
			}

			AsyncFocusSelectedItem();
		}

		private void ColumnHeader_Click(object sender, RoutedEventArgs e)
		{
			var header = e.OriginalSource as GridViewColumnHeader;
			if (header == null)
				return;

			string property = header.Tag as string;
			if (property == null)
				return;

			var listBox = (ListBox)sender;

			string originalContent = header.Column?.Header as string ?? property;
			ListSortDirection direction = ListSortDirection.Ascending;

			// for each list box, we save the column and the direction it was sorted.
			// if a columns was sorted, then restore the previous sorted header.
			if (ColumnSortState.TryGetValue(listBox, out var prev))
			{
				prev.Header.Content = prev.OriginalContent;
				// new direction?
				if (prev.Header == header && prev.Direction == ListSortDirection.Ascending)
					direction = ListSortDirection.Descending;
			}

			header.Content = originalContent + (direction == ListSortDirection.Ascending ? " ▲" : " ▼");
			ColumnSortState[listBox] = (header, originalContent, direction);

			var view = CollectionViewSource.GetDefaultView(listBox.ItemsSource);
			if (view != null)
			{
				view.SortDescriptions.Clear();
				view.SortDescriptions.Add(new SortDescription(property, direction));
			}

			// Scroll the selected item into view
			if (listBox.SelectedItem != null)
			{
				EventHandler scrollHandler = null;
				scrollHandler = (s2, e2) =>
				{
					listBox.LayoutUpdated -= scrollHandler;
					if (listBox.SelectedItem != null)
						listBox.ScrollIntoView(listBox.SelectedItem);
				};
				listBox.LayoutUpdated += scrollHandler;
			}
		}

		private void HidePanel()
		{
			ThreadHelper.ThrowIfNotOnUIThread();
			IVsWindowFrame ToolWindowFrame = (IVsWindowFrame)Window.Frame;
			ToolWindowFrame.Hide();
		}

		private void HidePanelAfterSelection()
		{
			if (!UnrealVSPackage.Instance.OptionsPage.FileBrowserKeepOpenAfterSelection)
			{
				HidePanel();
			}
		}

		private class FileBrowserSettings
		{
			public List<string> Bookmarks { get; set; } = new List<string>();
			public List<string> Recents { get; set; } = new List<string>();
		}


		private void SaveSolutionSettings()
		{
			if (String.IsNullOrEmpty(UnrealVsFileName))
			{
				return;
			}
			using (StreamWriter Writer = File.CreateText(UnrealVsFileName))
			{
				FileBrowserSettings Settings = new FileBrowserSettings();
				foreach (FileItem Item in BookmarkedFileItems)
					Settings.Bookmarks.Add(Item.File);
				foreach (FileItem Item in RecentFileItems)
					Settings.Recents.Add(Item.File);
				string Json = JsonSerializer.Serialize(Settings, new JsonSerializerOptions { WriteIndented = true });
				Writer.Write(Json);
			}
		}

		private void LoadSolutionSettings()
		{
			string SolutionFileName = UnrealVSPackage.Instance.DTE.Solution.FileName;
			if (String.IsNullOrEmpty(SolutionFileName))
			{
				BookmarkedFileItems.Clear();
				RecentFileItems.Clear();
				UnrealVsFileName = null;
				return;
			}
			UnrealVsFileName = SolutionFileName.Substring(0, SolutionFileName.Length - 3) + "unrealvs";
			if (!File.Exists(UnrealVsFileName))
			{
				return;
			}
			string Json = File.ReadAllText(UnrealVsFileName);
			if (String.IsNullOrEmpty(Json))
			{
				return;
			}
			FileBrowserSettings Settings = JsonSerializer.Deserialize<FileBrowserSettings>(Json, new JsonSerializerOptions { PropertyNameCaseInsensitive = true });
			if (Settings == null)
			{
				return;
			}


			BookmarkedFileItems.Clear();
			if (Settings.Bookmarks.Count == 0 && Settings.Recents.Count == 0)
				return;

			List<FileItem>[] FileItemLists = new[] { BookmarkedFileItems, RecentFileItems };
			List<string>[] SettingItems = new[] { Settings.Bookmarks, Settings.Recents };
			FileItem FoundItem;

			for (int i = 0; i != 2; ++i)
			{
				HashSet<FileItem> Added = new HashSet<FileItem>();
				foreach (string Item in SettingItems[i])
				{
					if (AllFileItemsLookup.TryGetValue(Item, out FoundItem))
					{
						if (Added.Add(FoundItem))
						{
							FileItemLists[i].Add(FoundItem);
						}
					}
				}
			}
		}

		private static ListBoxItem GetVisibleListViewElement(ListBox ListView, int Y)
		{
			HitTestResult HitTest = VisualTreeHelper.HitTest(ListView, new Point(10, Y));
			if (HitTest == null)
			{
				return null;
			}
			DependencyObject DepObj = HitTest.VisualHit as DependencyObject;
			if (DepObj == null)
			{
				return null;
			}
			DependencyObject Current = DepObj;
			while (Current != null && Current != ListView)
			{
				ListBoxItem BoxItem = Current as ListBoxItem;
				if (BoxItem != null)
				{
					return BoxItem;
				}
				Current = VisualTreeHelper.GetParent(Current);
			}
			return null;
		}

		private static char GetCharFromKey(Key Key)
		{
			char Ch = '\0';

			int VirtualKey = KeyInterop.VirtualKeyFromKey(Key);
			byte[] KeyboardState = new byte[256];
			NativeMethods.GetKeyboardState(KeyboardState);

			uint ScanCode = NativeMethods.MapVirtualKey((uint)VirtualKey, NativeMethods.MapType.MAPVK_VK_TO_VSC);
			StringBuilder Sb = new StringBuilder(2);

			int Result = NativeMethods.ToUnicode((uint)VirtualKey, ScanCode, KeyboardState, Sb, Sb.Capacity, 0);
			switch (Result)
			{
				case -1:
					break;
				case 0:
					break;
				case 1:
				default:
					Ch = Sb[0];
					break;
			}
			return Ch;
		}

		private class StackItem
		{
			public ProjectItems Items;
			public int Index;
			public Project Project;
		}

		private class SolutionTraverser
		{
			private List<FileItem> FileItems = new List<FileItem>();
			private Dictionary<string, string> ModuleRoots = new Dictionary<string, string>();

			private Stack<StackItem> ItemStack = new Stack<StackItem>();
			private int ProjectIndex;
			public int HandledItemCount { get; private set; }

			private readonly string[] Extensions = new string[] { ".build.cs", ".target.cs", ".uproject", ".uplugin" };

			public FileItem[] Update()
			{
				Stack<StackItem> TraversalStack = ItemStack;
				Projects Projects = UnrealVSPackage.Instance.DTE.Solution.Projects;
				int ProjectCount = Projects.Count;

				int TraverseCounter = 0;

				while (ProjectIndex < ProjectCount)
				{
					Project CurrentProject = Projects.Item(ProjectIndex + 1);

					StackItem CurrentItem;
					if (ItemStack.Count > 0)
					{
						CurrentItem = ItemStack.Pop();
					}
					else
					{
						CurrentItem = new StackItem() { Items = CurrentProject.ProjectItems, Project = CurrentProject };
					}

					while (true)
					{
						if (TraverseCounter > 5000)
						{
							ItemStack.Push(CurrentItem);
							return null;
						}

						if (CurrentItem.Items == null || CurrentItem.Index == CurrentItem.Items.Count)
						{
							if (TraversalStack.Count == 0)
							{
								break;
							}
							CurrentItem = TraversalStack.Pop();
							++CurrentItem.Index;
							continue;
						}

						ProjectItem ProjectItem = CurrentItem.Items.Item(CurrentItem.Index + 1);
						++TraverseCounter;

						for (short FileIndex = 0; FileIndex < ProjectItem.FileCount; ++FileIndex)
						{
							string FileName = ProjectItem.FileNames[(short)(FileIndex + 1)];
							if (FileName != null)
							{
								string Name = ProjectItem.Name;
								if (Name != FileName)
								{
									if (!FileName.EndsWith("\\")) // Skip folders
									{
										FileItem NewFileItem = new FileItem() { Name = Name, File = FileName, Module = CurrentItem.Project.Name };
										FileItems.Add(NewFileItem);

										foreach (string Extension in Extensions)
										{
											if (FileName.EndsWith(Extension, StringComparison.OrdinalIgnoreCase))
											{
												int LastBackslash = FileName.LastIndexOf('\\');
												string ModuleDir = FileName.Substring(0, LastBackslash);
												if (!ModuleRoots.ContainsKey(ModuleDir))
												{
													string ModuleName = FileName.Substring(LastBackslash + 1, FileName.Length - LastBackslash - Extension.Length - 1);
													ModuleRoots.Add(ModuleDir, ModuleName);
												}
												break;
											}
										}
										++HandledItemCount;
									}
								}
							}
						}

						if (ProjectItem.SubProject != null)
						{
							TraversalStack.Push(CurrentItem);
							CurrentItem = new StackItem() { Items = ProjectItem.SubProject.ProjectItems, Project = ProjectItem.SubProject };
							continue;
						}

						if (ProjectItem.ProjectItems != null)
						{
							TraversalStack.Push(CurrentItem);
							CurrentItem = new StackItem() { Items = ProjectItem.ProjectItems, Project = CurrentItem.Project };
							continue;
						}

						++CurrentItem.Index;
					}

					++ProjectIndex;
				}

				// Assign module to all file items
				foreach (FileItem Item in FileItems)
				{
					var Module = GetModuleForFile(Item.File);
					if (Module != null)
					{
						Item.Module = Module;
					}
				}

				return FileItems.ToArray();
			}

			private string GetModuleForFile(string FilePath)
			{
				int LastBackslash = FilePath.LastIndexOf('\\');
				while (LastBackslash > 0)
				{
					string dir = FilePath.Substring(0, LastBackslash);
					if (ModuleRoots.TryGetValue(dir, out string ModuleName))
					{
						return ModuleName;
					}
					LastBackslash = dir.LastIndexOf('\\');
				}
				return null;
			}
		}
	}
}