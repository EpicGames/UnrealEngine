// Copyright Epic Games, Inc. All Rights Reserved.

using Microsoft.VisualStudio.Shell;
using System;
using System.ComponentModel.Design;
using System.Diagnostics;
using System.IO;


namespace UnrealVS
{
	class GenerateProjectFiles
	{
		const int GenerateProjectFilesButtonID = 0x1100;


		public GenerateProjectFiles()
		{
			// GenerateProjectFilesButton
			{
				var CommandID = new CommandID(GuidList.UnrealVSCmdSet, GenerateProjectFilesButtonID);
				GenerateProjectFilesButtonCommand = new MenuCommand(new EventHandler(GenerateProjectFilesButtonHandler), CommandID);
				UnrealVSPackage.Instance.MenuCommandService.AddCommand(GenerateProjectFilesButtonCommand);
			}


			// Register to find out when the solution is closed or opened
			UnrealVSPackage.Instance.OnSolutionOpened += OnSolutionOpened;
			UnrealVSPackage.Instance.OnSolutionClosing += OnSolutionClosing;

			UpdateGenerateProjectFilesButtonState();
		}



		/// Called when 'GenerateProjectFiles' button is clicked
		void GenerateProjectFilesButtonHandler(object Sender, EventArgs Args)
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			// Don't allow a regen if we're already generating projects
			if (GenerateProjectFilesProcess == null)
			{
				// Figure out which project to rebuild
				string BatchFileName = GetBatchFileName();

				if (!String.IsNullOrEmpty(BatchFileName))
				{
					// Make sure the Output window is visible
					UnrealVSPackage.Instance.DTE.ExecuteCommand("View.Output");

					// Activate the output pane
					var Pane = UnrealVSPackage.Instance.GetOutputPane();
					if (Pane != null)
					{
						// Clear and activate the output pane.
						Pane.Clear();

						// @todo: Activating doesn't seem to really bring the pane to front like we would expect it to.
						Pane.Activate();
					}

					GenerateProjectFilesProcess = UnrealVSPackage.LaunchProgram(
						"cmd.exe", "/C " + GetSafeFilePath(BatchFileName),
						OnGenerateProjectFilesProcessExit,
						OnOutputFromGenerateProjectFilesProcess);
				}

				UpdateGenerateProjectFilesButtonState();
			}
		}


		/// Called when the GenerateProjectFiles process terminates
		void OnGenerateProjectFilesProcessExit(object Sender, EventArgs Args)
		{
			GenerateProjectFilesProcess = null;

			UpdateGenerateProjectFilesButtonState();
		}


		/// <summary>
		/// Updates the enabled/disabled state of the 'Generate Project Files' button
		/// </summary>
		void UpdateGenerateProjectFilesButtonState(bool ShouldForceDisable = false)
		{
			var CanGenerateProjects = false;

			if (!ShouldForceDisable)
			{
				// Can't launch when a process is already running
				if (GenerateProjectFilesProcess == null)
				{
					string BatchFileName = GetBatchFileName();
					if (BatchFileName != null)
					{
						CanGenerateProjects = true;
					}
					else
					{
						// No solution loaded, or solution doesn't look like Unreal
					}
				}
			}

			GenerateProjectFilesButtonCommand.Enabled = CanGenerateProjects;
		}


		/// <summary>
		/// Called when a solution is opened
		/// </summary>
		public void OnSolutionOpened()
		{
			UpdateGenerateProjectFilesButtonState();
		}


		/// <summary>
		/// Called when the solution is closed
		/// </summary>
		public void OnSolutionClosing()
		{
			UpdateGenerateProjectFilesButtonState(ShouldForceDisable: true);
		}


		/// <summary>
		/// Called when data is received from the GenerateProjectFiles process
		/// </summary>
		public void OnOutputFromGenerateProjectFilesProcess(object _Sender, DataReceivedEventArgs Args)
		{
			var AsyncTask = ThreadHelper.JoinableTaskFactory.RunAsync(async () =>
			{
				await ThreadHelper.JoinableTaskFactory.SwitchToMainThreadAsync();

				var Pane = UnrealVSPackage.Instance.GetOutputPane();
				if (Pane != null)
				{
					Pane.OutputStringThreadSafe(Args.Data + "\n");
				}
			});
			AsyncTask.Join();
		}

		private string GetBatchFileName()
		{
			// Check to see if we have the Unreal loaded
			if (UnrealVSPackage.Instance.IsUESolutionLoaded)
			{
				// We expect "GenerateProjectFiles.bat" to live in the same directory as the solution
				string BatchPath = Path.Combine(Path.GetDirectoryName(UnrealVSPackage.Instance.SolutionFilepath), "GenerateProjectFiles.bat");
				if (File.Exists(BatchPath))
				{
					return BatchPath;
				}
			}
			return null;
		}

		/// <summary>
		/// Wraps a file path in quotes if it contains a space character
		/// </summary>
		/// <param name="InPath"></param>
		/// <returns></returns>
		private string GetSafeFilePath(string InPath)
		{
			string WorkingPath = InPath;

			if (WorkingPath.Contains(" ") && !WorkingPath.Contains("\""))
			{
				WorkingPath = "\"" + WorkingPath + "\"";
			}
			return WorkingPath;
		}

		/// Active process for the "generate project files" command
		System.Diagnostics.Process GenerateProjectFilesProcess;

		/// Command for 'Generate Project Files' button
		readonly MenuCommand GenerateProjectFilesButtonCommand;
	}

}
