// Copyright Epic Games, Inc. All Rights Reserved.

// This file is used by Code Analysis to maintain SuppressMessage
// attributes that are applied to this project. Project-level
// suppressions either have no target or are given a specific target
// and scoped to a namespace, type, member, etc.
//
// To add a suppression to this file, right-click the message in the
// Error List, point to "Suppress Message(s)", and click "In Project
// Suppression File". You do not need to add suppressions to this
// file manually.

[assembly: System.Diagnostics.CodeAnalysis.SuppressMessage("Microsoft.Design", "CA1017:MarkAssembliesWithComVisible")]
