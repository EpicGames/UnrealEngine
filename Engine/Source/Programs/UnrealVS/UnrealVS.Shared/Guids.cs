// Copyright Epic Games, Inc. All Rights Reserved.

// Guids.cs
// MUST match guids.h
using System;

namespace UnrealVS
{
	static class GuidList
	{
		public const string UnrealVSPackageString = "ddbf523f-7eb6-4887-bd51-85a714ff87eb";
		public const string UnrealVSCmdSetString = "f10be850-eeec-4cef-ac64-1bc2cbe0d447";
		public const string UnrealVSPackageManString = "683EF1CF-9C97-46D7-BCFA-F12CA6FB77D6";
		public const string UnrealVSBatchBuildToolWindowPersistanceString = "9DE31877-5741-4B8B-9F3B-633555D7842B";
		public const string UnrealVSOptionsString = "FAC770BD-A983-4F40-9364-868D645AF16D";

		// spare generated GUIDs
		//[Guid("DFF26599-0F46-4A6B-8204-45CC89BAB09D")]
		//[Guid("E7E7B844-44CF-4DCE-A650-3F97A63C5743")]

		public static readonly Guid UnrealVSCmdSet = new Guid(UnrealVSCmdSetString);

		public static readonly Guid BatchBuildPaneGuid = new Guid("BBAC0710-4A2E-4BD3-9A39-4683FC6CFE86");

		// Known GUIDs - not defined in the SDK? (found at http://msdn.microsoft.com/en-us/library/hb23x61k(v=VS.80).aspx)
		public const string SolutionFolderGuidString = "{66A26720-8FB5-11D2-AA7E-00C04F688DDE}";
		public const string VBProjectKindGuidString = "{F184B08F-C81C-45F6-A57F-5ABD9991F28F}";
		public const string VCSharpProjectKindGuidString = "{FAE04EC0-301F-11D3-BF4B-00C04F79EFBC}";
		public const string VCProjectKindGuidString = "{8BC9CEB8-8B4A-11D0-8D11-00A0C91BC942}";
		public const string VJSharpProjectKindGuidString = "{E6FDF86B-F3D1-11D4-8576-0002A516ECE8}";
		public const string WebProjectKindGuidString = "{E24C65DC-7377-472b-9ABA-BC803B73C61A}";
	};
}