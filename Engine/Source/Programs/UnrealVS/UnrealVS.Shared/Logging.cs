// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Diagnostics;
using System.IO;

namespace UnrealVS
{
	static class Logging
	{
		private static readonly string LogFolderPathRoot =
			Path.Combine(Environment.GetFolderPath(Environment.SpecialFolder.LocalApplicationData), "Epic Games");

		private const string LogFileNameBase = "UnrealVS-log";
		private const string LogFileNameExt = ".txt";

		private static StreamWriter LogFile;
		private static bool bLoggingReady = false;

		private static string ExtensionName;
		private static string VersionString;

		private const int MaxFileSuffix = 64;

		public static void Initialize(string InExtensionName, string InVersionString)
		{
			ExtensionName = InExtensionName;
			VersionString = InVersionString;

			Initialize(0);
		}

		private static void Initialize(int FileSuffix)
		{
			try
			{
				string LogFolderPath = Path.Combine(LogFolderPathRoot, ExtensionName);

				if (!Directory.Exists(LogFolderPath))
				{
					Directory.CreateDirectory(LogFolderPath);
				}

				string LogFilePath = GetLogFilePath(LogFolderPath, FileSuffix);

				try
				{
					if (File.Exists(LogFilePath))
					{
						File.Delete(LogFilePath);
					}

					LogFile = new StreamWriter(LogFilePath);
					bLoggingReady = true;

					WriteLine(
						string.Format(
							"LOG: {0} {1} (started up at {2} - {3})",
							ExtensionName,
							VersionString,
							DateTime.Now.ToLongDateString(),
							DateTime.Now.ToLongTimeString()));

#if VS11
					WriteLine("Visual Studio 11 build");
#elif VS12
					WriteLine("Visual Studio 12 build");
#elif VS13
					WriteLine("Visual Studio 13 build");
#elif VS14
					WriteLine("Visual Studio 14 build");
#elif VS15
					WriteLine("Visual Studio 15 build");
#elif VS16
					WriteLine("Visual Studio 16 build");
#elif VS17
					WriteLine("Visual Studio 17 build");
#else
					WriteLine("UNKNOWN Visual Studio build");
#endif
				}
				catch (IOException)
				{
					if (MaxFileSuffix == FileSuffix) throw;

					Initialize(FileSuffix + 1);
				}
			}
			catch (Exception ex)
			{
				if (ex is ApplicationException) throw;

				bLoggingReady = false;
				throw new ApplicationException("Failed to init logging in UnrealVS", ex);
			}
		}

		private static string GetLogFilePath(string LogFolderPath, int FileSuffix)
		{
			string Suffix = string.Empty;
			if (FileSuffix > 0)
			{
				Suffix = string.Format("({0})", FileSuffix);
			}

			return LogFolderPath +
					Path.DirectorySeparatorChar +
					LogFileNameBase +
					Suffix +
					LogFileNameExt;
		}

		public static void Close()
		{
			if (!bLoggingReady) return;

			WriteLine(
				string.Format(
					"LOG: {0} {1} (closed at {2} - {3})",
					ExtensionName,
					VersionString,
					DateTime.Now.ToLongDateString(),
					DateTime.Now.ToLongTimeString()));

			LogFile.Close();
			LogFile = null;
			bLoggingReady = false;
		}

		public static void WriteLine(string Text)
		{
			Trace.WriteLine(Text);

			if (!bLoggingReady) return;

			LogFile.WriteLine(Text);
			LogFile.Flush();
		}
	}
}
