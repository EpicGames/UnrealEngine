// Copyright Epic Games, Inc. All Rights Reserved.

using EnvDTE;
using Microsoft.VisualStudio;
using Microsoft.VisualStudio.Shell;
using Microsoft.VisualStudio.Shell.Interop;
using Microsoft.VisualStudio.Text.Differencing;
using Microsoft.VisualStudio.Threading;
using System;
using System.Collections.Generic;
using System.ComponentModel.Design;
using System.Diagnostics;
using System.IO;
using System.Linq;
using System.Reflection.Metadata;
using System.Text;
using System.Text.RegularExpressions;
using System.Threading;
using System.Threading.Tasks;
using Process = System.Diagnostics.Process;

namespace UnrealVS
{
	using Task = System.Threading.Tasks.Task;

	internal class P4Commands : IDisposable
	{
		private const bool bPullWorkingDirectoryOn = true;
		private const bool bPullWorkingDirectoryOff = false;

		private const bool bOutputStdOutOn = true;
		private const bool bOutputStdOutOff = false;

		private const int P4SubMenuID = 0x3100;
		private const int P4CheckoutButtonID = 0x1450;
		private const int P4AnnotateButtonID = 0x1451;
		private const int P4ViewSelectedCLButtonID = 0x1452;
		private const int P4IntegrationAwareTimelapseButtonID = 0x1453;
		private const int P4DiffinVSButtonID = 0x1454;
		private const int P4GetLast10ChangesID = 0x1455;
		private const int P4ShowFileinP4VID = 0x1456;
		private const int P4FastReconcileCodeFilesID = 0x1457;
		private const int P4TimelapseButtonID = 0x1458;
		private const int P4RevisionGraphButtonID = 0x1459;
		private const int P4FileHistoryButtonID = 0x1460;
		private const int P4RevertButtonID = 0x1461;

		private OleMenuCommand SubMenuCommand;

		//private System.Diagnostics.Process ChildProcess;

		private List<Process> ChildProcessList = new List<Process>();

		private IVsOutputWindowPane P4OutputPane;
		private string P4WorkingDirectory;
		private bool bPullWorkingDirectorFromP4 = true;
		private static object bPullWorkingDirectorFromP4Lock = new object();

		// Exe paths
		private string P4Exe = "C:\\Program Files\\Perforce\\p4.exe";
		private string P4VCCmd = "C:\\Program Files\\Perforce\\p4vc.exe";
		private string P4VCCmdBat = "C:\\Program Files\\Perforce\\p4vc.bat";
		private string P4VExe = "C:\\Program Files\\Perforce\\p4v.exe";

		// user info
		private string Username = "";
		private string Port = "";
		private string Client = "";
		private string Stream = "";
		private string UserInfoComplete = "";
		private readonly object CheckoutQueueLock = new object();
		private readonly HashSet<string> CheckoutQueueFiles = new HashSet<string>(StringComparer.OrdinalIgnoreCase);
		private JoinableTask CheckoutTask;

		private class P4Command
		{
			public readonly OleMenuCommand ButtonCommand;

			public P4Command(int ButtonID, EventHandler ButtonHandler, Func<EnvDTE.Document, bool> QueryStatusHandler = null)
			{
				var cmd = new OleMenuCommand(new EventHandler(ButtonHandler), new CommandID(GuidList.UnrealVSCmdSet, ButtonID));
				cmd.BeforeQueryStatus += (s, e) =>
				{
					ThreadHelper.ThrowIfNotOnUIThread();
					DTE DTE = UnrealVSPackage.Instance.DTE;
					((MenuCommand)s).Enabled = DTE.ActiveDocument != null && (QueryStatusHandler == null || QueryStatusHandler(DTE.ActiveDocument));
				};
					
				ButtonCommand = cmd;
				UnrealVSPackage.Instance.MenuCommandService.AddCommand(ButtonCommand);
			}

			public void Toggle(bool Enabled)
			{
				ButtonCommand.Visible = ButtonCommand.Enabled = Enabled;
			}
		}

		private readonly List<P4Command> P4CommandsList = new List<P4Command>(16);

		private InterceptSave Interceptor;

		private bool IsSolutionLoaded()
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			DTE DTE = UnrealVSPackage.Instance.DTE;

			return DTE.Solution.FileName.Length > 0;
		}

		private string GetCheckoutQueueFileName()
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			DTE DTE = UnrealVSPackage.Instance.DTE;
			if (DTE.Solution.FileName.Length > 0)
			{
				return Path.Combine(Path.GetDirectoryName(DTE.Solution.FileName), ".p4checkout.txt");
			}

			return null;
		}

		private void OnQuickBuildSubMenuQuery(object sender, EventArgs e)
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			DTE DTE = UnrealVSPackage.Instance.DTE;

			bool bEnableCommands = UnrealVSPackage.Instance.OptionsPage.AllowUnrealVSP4;
			bool bSolutionLoaded = IsSolutionLoaded();

			var SenderSubMenuCommand = (OleMenuCommand)sender;

			SenderSubMenuCommand.Visible = SenderSubMenuCommand.Enabled = bEnableCommands & bSolutionLoaded;
		}

		public P4Commands()
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			// setup callbacks on IDE operations
			UnrealVSPackage.Instance.OptionsPage.OnOptionsChanged += OnOptionsChanged;
			UnrealVSPackage.Instance.OnSolutionOpened += SolutionOpened;
			UnrealVSPackage.Instance.OnSolutionClosed += SolutionClosed;

			// create specific output window for unrealvs.P4
			P4OutputPane = UnrealVSPackage.Instance.GetP4OutputPane();

			// figure out the P4VC path
			if (!File.Exists(P4VCCmd))
			{
				if (File.Exists(P4VCCmdBat))
				{
					P4VCCmd = P4VCCmdBat;
				}
				else
				{
					P4OutputPane.Activate();
					P4OutputPane.OutputStringThreadSafe($"1>------ P4VC not found, {P4VCCmd} or {P4VCCmdBat}{Environment.NewLine}");
					P4VCCmd = "";
				}

			}

			// add commands
			P4CommandsList.Add(new P4Command(P4CheckoutButtonID, P4CheckoutButtonHandler));
			P4CommandsList.Add(new P4Command(P4RevertButtonID, P4RevertButtonHandler));
			P4CommandsList.Add(new P4Command(P4AnnotateButtonID, P4AnnotateButtonHandler));
			P4CommandsList.Add(new P4Command(P4DiffinVSButtonID, P4DiffHandler));
			P4CommandsList.Add(new P4Command(P4GetLast10ChangesID, P4GetLast10ChangesHandler));
			P4CommandsList.Add(new P4Command(P4FastReconcileCodeFilesID, P4FastReconcileCodeFiles));

			if (P4VCCmd.Length > 1)
			{
				P4CommandsList.Add(new P4Command(P4IntegrationAwareTimelapseButtonID, P4IntegrationAwareTimeLapseHandler));
				P4CommandsList.Add(new P4Command(P4ShowFileinP4VID, P4ShowFileInP4VHandler));
				P4CommandsList.Add(new P4Command(P4FileHistoryButtonID, P4FileHistoryHandler));
				P4CommandsList.Add(new P4Command(P4RevisionGraphButtonID, P4RevisionGraphHandler));
				P4CommandsList.Add(new P4Command(P4TimelapseButtonID, P4TimelapseHandler));
				P4CommandsList.Add(new P4Command(P4ViewSelectedCLButtonID, P4ViewSelectedCLButtonHandler, (Document) =>
				{
					ThreadHelper.ThrowIfNotOnUIThread();
					TextSelection TextSel = (TextSelection)Document.Selection;
					return !string.IsNullOrEmpty(TextSel.Text) && int.TryParse(TextSel.Text, out _);
				}));
			}

			// add sub menu for commands
			SubMenuCommand = new OleMenuCommand(null, new CommandID(GuidList.UnrealVSCmdSet, P4SubMenuID));
			SubMenuCommand.BeforeQueryStatus += OnQuickBuildSubMenuQuery;
			UnrealVSPackage.Instance.MenuCommandService.AddCommand(SubMenuCommand);

			// Update the menu visibility to enforce user options.
			UpdateMenuOptions();

			var runningDocumentTable = new RunningDocumentTable(UnrealVSPackage.Instance);
			Interceptor = new InterceptSave(UnrealVSPackage.Instance.DTE, runningDocumentTable, this);

			runningDocumentTable.Advise(Interceptor);

			string CheckoutQueueFile = GetCheckoutQueueFileName();
			if (CheckoutQueueFile != null)
			{
				P4OutputPane.OutputStringThreadSafe("UnrealVS started" + Environment.NewLine);
				if (File.Exists(P4Exe))
				{
					P4OutputPane.OutputStringThreadSafe($"Using checkout queue: {CheckoutQueueFile}" + Environment.NewLine);
					PulseCheckoutQueue(CheckoutQueueFile);
				}
				else
				{
					P4OutputPane.OutputStringThreadSafe($"Unable to find Perforce executable ({P4Exe}). Perforce checkout queue will be disabled." + Environment.NewLine);
				}
			}
		}

		// Called when solutions are loaded or unloaded
		private void SolutionOpened()
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			// Update the menu visibility
			UpdateMenuOptions();

			string CheckoutQueueFile = GetCheckoutQueueFileName();
			if (CheckoutQueueFile != null)
			{
				P4OutputPane.OutputStringThreadSafe($"Solution opened. Using checkout queue: {CheckoutQueueFile}" + Environment.NewLine);
				PulseCheckoutQueue(CheckoutQueueFile);
			}
		}

		private void SolutionClosed()
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			// Update the menu visibility
			UpdateMenuOptions();

			// Clear any existing P4 working directory settings
			P4WorkingDirectory = "";
			bPullWorkingDirectorFromP4 = true;
		}

		private void UpdateMenuOptions()
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			DTE DTE = UnrealVSPackage.Instance.DTE;

			bool bEnableCommands = UnrealVSPackage.Instance.OptionsPage.AllowUnrealVSP4;
			bool bSolutionLoaded = IsSolutionLoaded();

			// update each menu item enabled command
			foreach (P4Command command in P4CommandsList)
			{
				command.Toggle(bSolutionLoaded && bEnableCommands);
			}

		}
		private void OnOptionsChanged(object Sender, EventArgs E)
		{
			ThreadHelper.ThrowIfNotOnUIThread();
			UpdateMenuOptions();
		}

		public void Dispose()
		{
			KillChildProcess();
		}

		private void KillChildProcess()
		{
			foreach (var OldProcess in ChildProcessList)
			{
				if (!OldProcess.HasExited)
				{
					OldProcess.Kill();
					OldProcess.WaitForExit();
				}
				OldProcess.Dispose();
			}
		}

		private void P4RevertButtonHandler(object Sender, EventArgs Args)
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			DTE DTE = UnrealVSPackage.Instance.DTE;

			// Check we've got a file open
			if (DTE.ActiveDocument == null)
			{
				Logging.WriteLine("P4Checkout called without an active document");

				P4OutputPane.Activate();
				P4OutputPane.OutputStringThreadSafe($"1>------ P4Revert called without an active document{Environment.NewLine}");

				return;
			}

			if (VSConstants.MessageBoxResult.IDOK == (VSConstants.MessageBoxResult) VsShellUtilities.ShowMessageBox(UnrealVSPackage.Instance, string.Empty, $"Are you sure you want to revert {DTE.ActiveDocument.Name}?", OLEMSGICON.OLEMSGICON_QUERY, OLEMSGBUTTON.OLEMSGBUTTON_OKCANCEL, OLEMSGDEFBUTTON.OLEMSGDEFBUTTON_FIRST))
			{
				TryP4Command($"revert {DTE.ActiveDocument.FullName}", out _, out _);
			}
		}

		private void P4CheckoutButtonHandler(object Sender, EventArgs Args)
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			DTE DTE = UnrealVSPackage.Instance.DTE;

			// Check we've got a file open
			if (DTE.ActiveDocument == null)
			{
				Logging.WriteLine("P4Checkout called without an active document");

				P4OutputPane.Activate();
				P4OutputPane.OutputStringThreadSafe($"1>------ P4Checkout called without an active document{Environment.NewLine}");

				return;
			}

			OpenForEdit(DTE.ActiveDocument.FullName);
		}
		private void P4AnnotateButtonHandler(object Sender, EventArgs Args)
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			// Debug.ListCallStack

			DTE DTE = UnrealVSPackage.Instance.DTE;

			P4OutputPane.Activate();

			// Check we've got a file open
			if (DTE.ActiveDocument == null)
			{
				Logging.WriteLine("P4Annotate called without an active document");

				P4OutputPane.OutputStringThreadSafe($"1>------ P4Annotate called without an active document{Environment.NewLine}");

				return;
			}

			string CaptureP4Output;
			string CaptureP4StdErr;

			// Call annotate itself
			bool bResult = TryP4Command($"annotate -TcIqu \"{DTE.ActiveDocument.FullName}", out CaptureP4Output, out CaptureP4StdErr, bPullWorkingDirectoryOn, bOutputStdOutOff);

			if (!bResult || CaptureP4StdErr.Length > 0)
			{
				P4OutputPane.OutputStringThreadSafe($"1>------ P4Annotate call failed");
				return;
			}

			// Extract the current document line number and the first line of text within it
			string FirstLine;
			int CurrentLine = 0;
			{
				TextDocument CurrentDocument = (TextDocument)(DTE.ActiveDocument.Object("TextDocument"));

				var EditPoint = CurrentDocument.StartPoint.CreateEditPoint();
				string DocumentContents = EditPoint.GetText(CurrentDocument.EndPoint);
				FirstLine = DocumentContents.Split('\n')[0];


				TextSelection TextSel = DTE.ActiveWindow.Selection as TextSelection;
				if (TextSel != null)
				{
					CurrentLine = TextSel.CurrentLine;
				}
			}

			// Pre-process the output to comment out the additions thus allowing
			// code to use correct syntax coloring - helps enormously with visualization
			StringBuilder EditedCopy = new StringBuilder();
			{
				// replace 
				//       13149436:            First.Last 2020/05/04 
				//
				// with
				// /*	13149436:            First.Last 2020/05/04*/

				// This is the per line offset of the annotation added to the document
				int AnnotateOffset = CaptureP4Output.IndexOf(FirstLine);

				string[] AnnotateLines = CaptureP4Output.Split('\n');

				foreach (string Line in AnnotateLines)
				{
					if (Line.Length > AnnotateOffset)
					{
						string EditedLine = Line.Insert(AnnotateOffset, "*/");
						EditedLine = EditedLine.Insert(0, "/*");

						EditedCopy.Append(EditedLine);
					}
					else
					{
						EditedCopy.Append(Line);
					}

				}
			}

			// Replace GetTempPath with the UBT intermediate folder if we have access
			string TempPath = Path.GetTempPath();
			string TempFileName = Path.GetFileNameWithoutExtension(DTE.ActiveDocument.FullName) + "_annotate" + Path.GetExtension(DTE.ActiveDocument.FullName);
			string TempFilePath = Path.Combine(TempPath, TempFileName);

			// Write out our temp file
			File.WriteAllText(TempFilePath, EditedCopy.ToString());

			// Open it, activate it and move to the line the user focused to execute the command
			DTE.ExecuteCommand("File.OpenFile", $"\"{TempFilePath}\"");
			DTE.ActiveDocument.Activate();

			TextSelection NewTextSel = DTE.ActiveWindow.Selection as TextSelection;
			if (NewTextSel != null)
			{
				NewTextSel.GotoLine(CurrentLine, false);
			}
		}
		private void P4ViewSelectedCLButtonHandler(object Sender, EventArgs Args)
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			DTE DTE = UnrealVSPackage.Instance.DTE;

			if (DTE.ActiveDocument == null)
			{
				return;
			}

			TextSelection TextSel = (TextSelection)DTE.ActiveDocument.Selection;

			if(Int32.TryParse(TextSel.Text, out int ChangeList) && ChangeList > 0)
			{
				TryP4VCCommand($"Change {ChangeList}");
			}
		}

		private void P4TimelapseHandler(object sender, EventArgs args)
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			DTE DTE = UnrealVSPackage.Instance.DTE;

			if (DTE.ActiveDocument == null)
			{
				Logging.WriteLine("P4Timelapse called without an active document");
				P4OutputPane.OutputStringThreadSafe($"1>------ P4Timelapse called without an active document{Environment.NewLine}");
				return;
			}

			string Command = "timelapse ";

			TextDocument CurrentDocument = (TextDocument) DTE.ActiveDocument.Object("TextDocument");
			if (CurrentDocument != null)
			{
				TextSelection TextSel = (TextSelection) DTE.ActiveDocument.Selection;
				if (TextSel != null)
				{
					Command += $"-l {TextSel.CurrentLine} ";
				}
			}

			Command += $"\"{DTE.ActiveDocument.FullName}\"";

			TryP4VCCommand(Command, true);
		}

		private void P4RevisionGraphHandler(object sender, EventArgs args)
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			DTE DTE = UnrealVSPackage.Instance.DTE;

			if (DTE.ActiveDocument == null)
			{
				Logging.WriteLine("P4Timelapse called without an active document");
				P4OutputPane.OutputStringThreadSafe($"1>------ P4RevisionGraph called without an active document{Environment.NewLine}");
				return;
			}

			string Command = $"revgraph \"{DTE.ActiveDocument.FullName}\"";
			TryP4VCCommand(Command, true);
		}

		private void P4FileHistoryHandler(object sender, EventArgs args)
		{
			if (P4VCCmd.Length == 0)
			{
				return;
			}

			ThreadHelper.ThrowIfNotOnUIThread();

			DTE DTE = UnrealVSPackage.Instance.DTE;

			if (DTE.ActiveDocument == null)
			{
				Logging.WriteLine("P4FileHistory called without an active document");
				P4OutputPane.OutputStringThreadSafe($"1>------ P4FileHistory called without an active document{Environment.NewLine}");
				return;
			}

			string Command = $"history \"{DTE.ActiveDocument.FullName}\"";
			TryP4VCCommand(Command);
		}

		private void P4IntegrationAwareTimeLapseHandler(object Sender, EventArgs Args)
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			DTE DTE = UnrealVSPackage.Instance.DTE;

			if (DTE.ActiveDocument == null)
			{
				Logging.WriteLine("P4IntegrationAwareTimeLapse called without an active document");

				P4OutputPane.OutputStringThreadSafe($"1>------ P4IntegrationAwareTimeLapse called without an active document{Environment.NewLine}");

				return;
			}

			string Command = $"-win 0 {UserInfoComplete} -cmd \"annotate -i \"{DTE.ActiveDocument.FullName}\"\"";

			TryP4VCommand(Command);
		}

		private void ChangeDiffSetting()
		{
			if (UnrealVSPackage.Instance.OptionsPage.AllowUnrealVSOverrideDiffSettings)
			{
				bool bMargin = true;
				UnrealVSPackage.Instance.EditorOptionsFactory.GlobalOptions.SetOptionValue("Diff/View/ShowDiffOverviewMargin", bMargin);

				DifferenceHighlightMode HighlightMode = (DifferenceHighlightMode)3;
				UnrealVSPackage.Instance.EditorOptionsFactory.GlobalOptions.SetOptionValue("Diff/View/HighlightMode", HighlightMode);
			}
		}

		private void P4DiffHandler(object Sender, EventArgs Args)
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			DTE DTE = UnrealVSPackage.Instance.DTE;

			if (DTE.ActiveDocument == null)
			{
				Logging.WriteLine("P4DiffInVSHandler called without an active document");

				P4OutputPane.OutputStringThreadSafe($"1>------ P4DiffInVSHandler called without an active document{Environment.NewLine}");

				return;
			}

			if(UnrealVSPackage.Instance.OptionsPage.UseP4VDiff && P4VCCmd.Length > 1)
			{
				TryP4VCCommand($"diffhave \"{DTE.ActiveDocument.FullName}\"");
				return;
			}

			ChangeDiffSetting();

			string CaptureP4Output = "";

			// get the HAVE revision
			// p4 fstat -T "haveRev" -Olp //UE5/Main/Engine/Source/Programs/UnrealVS/UnrealVS.Shared/P4Commands.cs
			if (TryP4Command($"fstat -T \"haveRev,depotFile\" -Olp \"{DTE.ActiveDocument.FullName}\"", out CaptureP4Output, out _))
			{
				// expect output of the form
				//		"... haveRev 5"
				//		"... depotFile //UE5/Main/Engine/Source/Programs/UnrealVS/UnrealVS.Shared/P4Commands.cs
				Regex HavePattern = new Regex(@"... haveRev (?<Have>.*)$", RegexOptions.Compiled | RegexOptions.Multiline);
				Regex DepotPathPattern = new Regex(@"... depotFile (?<depotFile>.*)$", RegexOptions.Compiled | RegexOptions.Multiline);
				System.Text.RegularExpressions.Match HaveMatch = HavePattern.Match(CaptureP4Output);
				System.Text.RegularExpressions.Match PathMatch = DepotPathPattern.Match(CaptureP4Output);
				int HaveRev = Int32.Parse(HaveMatch.Groups["Have"].Value.Trim());
				string depotPath = PathMatch.Groups["depotFile"].Value.Trim();

				// Generate the Temp filename
				string TempPath = Path.GetTempPath();
				string TempFileName = Path.GetFileNameWithoutExtension(DTE.ActiveDocument.FullName) + "$" + HaveRev.ToString() + Path.GetExtension(DTE.ActiveDocument.FullName);
				string TempFilePath = Path.Combine(TempPath, TempFileName);

				// sync the HAVE revision to a file
				// p4 print //UE5/Main/Engine/Source/Programs/UnrealVS/UnrealVS.Shared/P4Commands.cs#5 >> file

				string CaptureP4Output2 = "";

				string VersionPath = $"{depotPath}#{HaveRev}";
				if (TryP4Command($"-q print \"{VersionPath}\"", out CaptureP4Output2, out _, bPullWorkingDirectoryOn, bOutputStdOutOff))
				{
					File.WriteAllText(TempFilePath, CaptureP4Output2);

					// Tools.DiffFiles SourceFile, TargetFile, [SourceDisplayName],[TargetDisplayName]
					DTE.ExecuteCommand("Tools.DiffFiles", $"\"{TempFilePath}\" \"{DTE.ActiveDocument.FullName}\" \"{VersionPath}\" \"{DTE.ActiveDocument.FullName}\"");
				}
			}
		}

		private void P4GetLast10ChangesHandler(object Sender, EventArgs Args)
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			DTE DTE = UnrealVSPackage.Instance.DTE;

			string TempPath = Path.GetTempPath();

			// generate the changes list
			string ChangesTempFileName = Path.GetFileNameWithoutExtension(DTE.ActiveDocument.FullName) + "_last_10_changelists" + Path.GetExtension(DTE.ActiveDocument.FullName);
			string ChangesTempFilePath = Path.Combine(TempPath, ChangesTempFileName);

			string CaptureP4Output = "";
			if (TryP4Command($"changes -itls submitted -m 10 \"{DTE.ActiveDocument.FullName}\"", out CaptureP4Output, out _, bOutputStdOutOff))
			{
				File.WriteAllText(ChangesTempFilePath, CaptureP4Output);

				DTE.ExecuteCommand("File.OpenFile", $"\"{ChangesTempFilePath}\"");
				DTE.ActiveDocument.Activate();
				DTE.ExecuteCommand("Window.NewVerticalTabGroup");
			}
		}

		private void P4FastReconcileCodeFiles(object Sender, EventArgs Args)
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			DTE DTE = UnrealVSPackage.Instance.DTE;

			string[] DefaultExtensions =
			{
				"c*",
				"h*",
				"ini",
				"uproject",
				"uplugin"
			};

			string[] Extensions = UnrealVSPackage.Instance.OptionsPage.ReconcileExtensions.Split(';');

			if (Extensions.Length < 1)
			{
				Extensions = DefaultExtensions;
			}

			// generate the changes list
			string ReconcileDepotPaths = "";
			string Preview = "-n";

			if (string.IsNullOrWhiteSpace(P4WorkingDirectory))
			{
				PullWorkingDirectory(bPullWorkingDirectoryOn);
			}

			foreach (string Ext in Extensions)
			{
				ReconcileDepotPaths += P4WorkingDirectory + Path.DirectorySeparatorChar + "...." + Ext + " ";
			}

			if (UnrealVSPackage.Instance.OptionsPage.AllowReconcileToMarkForEdit)
			{
				Preview = "";
			}
			else
			{
				P4OutputPane.OutputStringThreadSafe($"PREVIEW{Environment.NewLine}");
			}

			P4OutputPane.OutputStringThreadSafe($"Running Async Reconcile on : {ReconcileDepotPaths}{Environment.NewLine}");
			P4OutputPane.Activate();

			_ = TryP4CommandAsync($"reconcile -e -m {Preview} {ReconcileDepotPaths}");
		}

		private void P4ShowFileInP4VHandler(object Sender, EventArgs Args)
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			DTE DTE = UnrealVSPackage.Instance.DTE;

			if (DTE.ActiveDocument == null)
			{
				return;
			}

			if(UserInfoComplete.Length == 0)
			{
				SetUserInfoStrings();
			}

			if (Username.Length > 0 && Port.Length > 0 && Client.Length > 0)
			{
				TryP4VCommand($"-p {Port} -u {Username} -c {Client} -s \"{DTE.ActiveDocument.FullName}\"");
			}
			else
			{
				TryP4VCommand($"-s \"{DTE.ActiveDocument.FullName}\"");
			}
		}

		public void OpenForEdit(string FileName, bool CheckOptionsFlag = false)
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			// if the callee requested it, honor the enabling of autocheckout option - return if its off
			if (CheckOptionsFlag && !UnrealVSPackage.Instance.OptionsPage.AllowUnrealVSCheckoutOnEdit)
			{
				//P4OutputPane.OutputStringThreadSafe($"autocheckout disabled: {FileName}");
				return;
			}

			// Don't open for edit if the file is already writable, unless the command was user-triggered
			if (!File.Exists(FileName) ||
 				(CheckOptionsFlag && !File.GetAttributes(FileName).HasFlag(FileAttributes.ReadOnly)))
			{
				P4OutputPane.OutputStringThreadSafe($"already writeable: {FileName}" + Environment.NewLine);
				return;
			}

			if (UnrealVSPackage.Instance.OptionsPage.AllowAsyncP4Checkout)
			{
				string queueFile = GetCheckoutQueueFileName();
				if (queueFile == null)
				{
					P4OutputPane.OutputStringThreadSafe("Unable to get path to checkout queue file. No solution loaded?" + Environment.NewLine);
					return;
				}

				// Add the file to the queue for checking out before modifying it. This allows us to recover in the case of a P4 outage or VS exit.
				Logging.WriteLine($"Adding async checkout of {FileName} to {queueFile}");
				lock (CheckoutQueueLock)
				{
					try
					{
						File.AppendAllLines(queueFile, new[] { FileName });
					}
					catch (Exception ex)
					{
						P4OutputPane.OutputStringThreadSafe($"Unable to update {queueFile} ({ex.Message})" + Environment.NewLine);
						return;
					}
				}

				//P4OutputPane.OutputStringThreadSafe($"Marking file as writeable: {FileName}");
				// Mark the file as writeable
				FileAttributes NewAttributes = File.GetAttributes(FileName) & ~FileAttributes.ReadOnly;
				File.SetAttributes(FileName, NewAttributes);

				// Start a background thread to update the queue
				PulseCheckoutQueue(queueFile);
			}
			else
			{
				string StdOut, StdErr;

				// sync edit command - will lock the UI but be safer (no risk of a locally writeable file that is not checked out)
				bool Success = TryP4Command($"edit \"{FileName}\"", out StdOut, out StdErr);

				if (!Success && StdErr.Contains("file(s) not on client."))
				{
					P4OutputPane.OutputStringThreadSafe($"{FileName} has not been added yet, adding now.{Environment.NewLine}");
					TryP4Command($"add \"{FileName}\"", out _, out _);
				}
			}
		}

		[System.Diagnostics.CodeAnalysis.SuppressMessage("Usage", "VSTHRD003:Avoid awaiting foreign Tasks", Justification = "TODO")]
		[System.Diagnostics.CodeAnalysis.SuppressMessage("Reliability", "VSSDK007:ThreadHelper.JoinableTaskFactory.RunAsync", Justification = "TODO")]
		private void PulseCheckoutQueue(string QueueFile)
		{
			TaskCompletionSource<bool> StartTask = null;
			lock (CheckoutQueueLock)
			{
				CheckoutQueueFiles.Add(QueueFile);

				if (CheckoutTask == null)
				{
					Logging.WriteLine($"Starting checkout task.");
					StartTask = new TaskCompletionSource<bool>();
					CheckoutTask = ThreadHelper.JoinableTaskFactory.RunAsync(async () => { await StartTask.Task; await UpdateCheckoutQueueAsync(); });
				}
			}
			StartTask?.SetResult(true);
		}

		private async Task UpdateCheckoutQueueAsync()
		{
			for (; ; )
			{
				string QueueFile = null;
				try
				{
					// Read the list of files that need checking out
					string[] Lines;
					lock (CheckoutQueueLock)
					{
						while (QueueFile == null || !File.Exists(QueueFile))
						{
							QueueFile = CheckoutQueueFiles.FirstOrDefault();
							if (QueueFile == null)
							{
								Logging.WriteLine($"Stopping checkout task; setting CheckoutTask to null.");
								CheckoutTask = null;
								return;
							}
							CheckoutQueueFiles.Remove(QueueFile);
						}
						Lines = File.ReadAllLines(QueueFile);
					}

					// Open each file for edit
					bool Pause = false;
					foreach (string Line in Lines)
					{
						string FileName = Line.Trim();
						if (FileName.Length > 0)
						{
							bool Success = false;

							(bool EditSuccess, string StdOut, string StdErr) = await TryP4CommandExAsync(P4Exe, $"edit \"{FileName}\"");
							if (EditSuccess)
							{
								Success = true;

								Logging.WriteLine($"Performed async checkout of {FileName}.");
								lock (CheckoutQueueLock)
								{
									string[] NewLines = File.ReadAllLines(QueueFile);
									File.WriteAllLines(QueueFile, NewLines.Where(x => !x.Equals(Line, StringComparison.Ordinal)));
								}
							}

							if (!EditSuccess && StdErr.Contains("file(s) not on client."))
							{
								await ThreadHelper.JoinableTaskFactory.RunAsync(async () =>
								{
									await ThreadHelper.JoinableTaskFactory.SwitchToMainThreadAsync();
									P4OutputPane.OutputStringThreadSafe($"{FileName} has not been added yet, adding now.{Environment.NewLine}");
								});
								(bool AddSuccess, string _, string _) = await TryP4CommandExAsync(P4Exe, $"add \"{FileName}\"");

								if (AddSuccess)
								{
									Success = true;

									Logging.WriteLine($"Performed async add of {FileName}.");
									lock (CheckoutQueueLock)
									{
										string[] NewLines = File.ReadAllLines(QueueFile);
										File.WriteAllLines(QueueFile, NewLines.Where(x => !x.Equals(Line, StringComparison.Ordinal)));
									}
								}
							}

							if (!Success)
							{
								// Re-add to checkout queue, to force another loop
								Logging.WriteLine($"Unable to check out file {FileName}. Will retry.");
								CheckoutQueueFiles.Add(QueueFile);
								Pause = true;
							}
						}
					}

					// Check if we've looped over everything and should pause
					if (Pause)
					{
						await Task.Delay(TimeSpan.FromSeconds(10.0));
					}
				}
				catch (Exception ex)
				{
					Logging.WriteLine($"Unable to update checkout queue: {ex}");

					if (QueueFile != null)
					{
						lock (CheckoutQueueLock)
						{
							CheckoutQueueFiles.Add(QueueFile);
						}
					}

					await Task.Delay(TimeSpan.FromSeconds(10.0));
				}
			}
		}

		private string ReadWorkingDirectorFromP4Config()
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			string WorkingDirectory = "";
			//------P4 Operation started
			//P4CLIENT = andrew.firth_private_frosty2(config 'd:\p4\frosty\p4config.txt')
			//P4CONFIG = p4config.txt(set)(config 'd:\p4\frosty\p4config.txt')
			//P4EDITOR = C:\windows\SysWOW64\notepad.exe(set)
			//P4PORT = perforce:1666(config 'd:\p4\frosty\p4config.txt')
			//P4USER = andrew.firth(config 'd:\p4\frosty\p4config.txt')
			//P4_perforce:1666_CHARSET = none(set)

			string CaptureP4Output = "";
			TryP4Command("set", out CaptureP4Output, out _, bPullWorkingDirectoryOff);

			bool bSuccess = false;

			string[] lines = CaptureP4Output.Split('\n');
			foreach (string Line in lines)
			{
				string TrimLine = Line.Trim();

				// Match this in https://regex101.com/ to debug it. Basically it's words=anything(anything'path-to-config-file'anything).
				System.Text.RegularExpressions.Match Match = Regex.Match(TrimLine, @"^\w+=.*\(.*'(.*)'.*\)$");
				if (Match.Success)
				{
					WorkingDirectory = Path.GetDirectoryName(Match.Groups[1].Value);
					if (Directory.Exists(WorkingDirectory))
					{
						bSuccess = true;
						break;
					}
				}
			}

			if (!bSuccess)
			{
				P4OutputPane.OutputStringThreadSafe($"Attempt to pull the P4WorkingDirectory from 'p4 set' failed.  Have you run 'RunUAT P4WriteConfig' in your root directory?{Environment.NewLine}");
			}

			return WorkingDirectory;
		}

		void SetUserInfoStrings()
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			string CaptureP4Output;
			TryP4Command($"-s -L \"{P4WorkingDirectory}\" info", out CaptureP4Output, out _, bPullWorkingDirectoryOff);

			Regex UserPattern = new Regex(@"User name: (?<user>.*)$", RegexOptions.Compiled | RegexOptions.Multiline);
			Regex PortPattern = new Regex(@"Server address: (?<port>.*)$", RegexOptions.Compiled | RegexOptions.Multiline);
			Regex ClientPattern = new Regex(@"Client name: (?<client>.*)$", RegexOptions.Compiled | RegexOptions.Multiline);
			Regex ClientStream = new Regex(@"Client stream: (?<stream>.*)$", RegexOptions.Compiled | RegexOptions.Multiline);
			Regex ServerEncryption = new Regex(@"Server encryption: (?<encryption>.*)$", RegexOptions.Compiled | RegexOptions.Multiline);

			System.Text.RegularExpressions.Match UserMatch = UserPattern.Match(CaptureP4Output);
			System.Text.RegularExpressions.Match PortMatch = PortPattern.Match(CaptureP4Output);
			System.Text.RegularExpressions.Match ClientMatch = ClientPattern.Match(CaptureP4Output);
			System.Text.RegularExpressions.Match StreamMatch = ClientStream.Match(CaptureP4Output);
			System.Text.RegularExpressions.Match EncryptionMatch = ServerEncryption.Match(CaptureP4Output);

			Port = PortMatch.Groups["port"].Value.Trim();
			Username = UserMatch.Groups["user"].Value.Trim();
			Client = ClientMatch.Groups["client"].Value.Trim();
			Stream = StreamMatch.Groups["stream"].Value.Trim();

			if (EncryptionMatch.Groups["encryption"].Value.Trim().Equals("encrypted", StringComparison.OrdinalIgnoreCase) && !Port.StartsWith("ssl:", StringComparison.OrdinalIgnoreCase))
			{
				Port = "ssl:" + Port;
			}

			UserInfoComplete = string.Format(" -p {0} -u {1} -c {2} ", Port, Username, Client);

			P4OutputPane.OutputStringThreadSafe("GetUserInfoStringFull : " + UserInfoComplete + Environment.NewLine);
		}

		private void PullWorkingDirectory(bool bPullfromP4Settings)
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			lock(bPullWorkingDirectorFromP4Lock)
			{ 
				if (IsSolutionLoaded() && (P4WorkingDirectory == null || P4WorkingDirectory.Length < 2))
				{
					DTE DTE = UnrealVSPackage.Instance.DTE;

					// use the current solution folder as a temp working directory
					P4WorkingDirectory = Path.GetDirectoryName(DTE.Solution.FileName);

					// SetUserInfoStrings 
					SetUserInfoStrings();
				}

				// if the callee wants us to pull a CWD and it wasn't already done
				// pull it from the p4 config now
				if (bPullfromP4Settings && bPullWorkingDirectorFromP4)
				{
					string NewWorkingDirectory = ReadWorkingDirectorFromP4Config();

					if (NewWorkingDirectory.Length > 1)
					{
						P4WorkingDirectory = NewWorkingDirectory;
						bPullWorkingDirectorFromP4 = false;

						P4OutputPane.OutputStringThreadSafe($"P4WorkingDirectory set to '{P4WorkingDirectory}'{Environment.NewLine}");
					}
					else
					{
						P4OutputPane.OutputStringThreadSafe($"P4WorkingDirectory set failed {Environment.NewLine}");
					}
				}
			}
		}

		public async Task<bool> TryP4CommandAsync(string CommandLine, bool bPullWorkingDirectoryNow = bPullWorkingDirectoryOn, bool bOutputStdOut = bOutputStdOutOn)
		{
			(bool Result, _, _) = await TryP4CommandExAsync(P4Exe, CommandLine, bPullWorkingDirectoryNow, bOutputStdOut);
			return Result;
		}

		private bool TryP4Command(string CommandLine, out string CaptureP4Output, out string CaptureP4StdErr, bool bPullWorkingDirectoryNow = bPullWorkingDirectoryOn, bool bOutputStdOut = bOutputStdOutOn)
		{
			return TryP4CommandEx(P4Exe, CommandLine, out CaptureP4Output, out CaptureP4StdErr, bPullWorkingDirectoryNow, bOutputStdOut);
		}

		private bool TryP4VCCommand(string CommandLine, bool bPullWorkingDirectoryNow = bPullWorkingDirectoryOn, bool bOutputStdOut = bOutputStdOutOn)
		{
			if (P4VCCmd.Length > 1)
			{
				return TryP4CommandEx(P4VCCmd, CommandLine, out _, out _, bPullWorkingDirectoryNow, bOutputStdOut);
			}

			return false;
		}

		private void TryP4VCommand(string CommandLine, bool bPullWorkingDirectoryNow = bPullWorkingDirectoryOn, bool bOutputStdOut = bOutputStdOutOn)
		{
			_ = TryP4CommandExAsync(P4VExe, CommandLine, bPullWorkingDirectoryNow, bOutputStdOut);
		}

		private bool TryP4CommandEx(string CmdPath, string CommandLine, out string CaptureP4StdOut, out string CaptureP4StdErr, bool bPullWorkingDirectoryNow = bPullWorkingDirectoryOn, bool bOutputStdOut = bOutputStdOutOn)
		{
			(bool Result, string StdOut, string StdErr) = ThreadHelper.JoinableTaskFactory.Run(() => TryP4CommandExAsync(CmdPath, CommandLine, bPullWorkingDirectoryNow, bOutputStdOut));

			CaptureP4StdOut = StdOut;
			CaptureP4StdErr = StdErr;

			return Result;
		}

		private async Task<(bool Result, string StdOut, string StdErr)> TryP4CommandExAsync(string CmdPath, string CommandLine, bool bPullWorkingDirectoryNow = bPullWorkingDirectoryOn, bool bOutputStdOut = bOutputStdOutOn)
		{
			await ThreadHelper.JoinableTaskFactory.RunAsync(async () =>
			{
				await ThreadHelper.JoinableTaskFactory.SwitchToMainThreadAsync();
				PullWorkingDirectory(bPullWorkingDirectoryNow);

				// Set up the output pane
				if (UnrealVSPackage.Instance.OptionsPage.ForceOutputWindow)
				{
					P4OutputPane.Activate();
				}

				P4OutputPane.OutputStringThreadSafe($"Running \"{CmdPath}\" {CommandLine}{Environment.NewLine}");
			});

			const int PollIntervalMs = 100;
			const int TimeOutS = 10;

			bool bTimedOut = false;

			StringBuilder StdOutSB = new StringBuilder();
			StringBuilder StdErrSB = new StringBuilder();
			DateTime Start = DateTime.Now;

			// Spawn the new process
			System.Diagnostics.Process ChildProcess = new System.Diagnostics.Process()
			{
				StartInfo = new ProcessStartInfo()
				{
					FileName = CmdPath,
					Arguments = CommandLine,
					WorkingDirectory = P4WorkingDirectory,
					UseShellExecute = false,
					RedirectStandardOutput = true,
					RedirectStandardError = true,
					CreateNoWindow = true
				}
			};
			// Create a delegate for handling output messages
			ChildProcess.OutputDataReceived += (s, a) => { if (a.Data != null) StdOutSB.AppendLine(a.Data); };
			ChildProcess.ErrorDataReceived += (s, a) => { if (a.Data != null) StdErrSB.AppendLine(a.Data); };
			ChildProcess.EnableRaisingEvents = true;

			TaskCompletionSource<bool> ProcessExitTaskSource = new TaskCompletionSource<bool>();
			ChildProcess.Exited += (s, a) =>
			{
				ProcessExitTaskSource.TrySetResult(true);
			};

			lock (ChildProcessList)
			{
				ChildProcessList.Add(ChildProcess);
			}

			Stopwatch ProcessStartTime = new Stopwatch();
			ProcessStartTime.Start();

			try
			{
				ChildProcess.Start();
			}
			catch (Exception ex)
			{
				return (false, "", $"Unable to launch {CmdPath} ({ex.Message})");
			}

			ChildProcess.BeginOutputReadLine();
			ChildProcess.BeginErrorReadLine();

			await ProcessExitTaskSource.Task;

			while (!ChildProcess.WaitForExit(PollIntervalMs))
			{
				if (ProcessStartTime.Elapsed.Seconds >= TimeOutS)
				{
					bTimedOut = true;
					break;
				}
			}

			lock (ChildProcessList)
			{
				ChildProcessList.Remove(ChildProcess);
			}

			if (!bTimedOut)
			{
				string StdOut = StdOutSB.ToString();
				string StdErr = StdErrSB.ToString();

				await ThreadHelper.JoinableTaskFactory.RunAsync(async () =>
				{
					await ThreadHelper.JoinableTaskFactory.SwitchToMainThreadAsync();
					if (StdOut.Length > 0 && bOutputStdOut)
					{
						P4OutputPane.OutputStringThreadSafe(StdOut);
					}
					if (StdErr.Length > 0)
					{
						P4OutputPane.OutputStringThreadSafe(StdErr);
					}

					TimeSpan Duration = DateTime.Now - Start;
					P4OutputPane.OutputStringThreadSafe($"complete {Duration.TotalSeconds.ToString()} {Environment.NewLine}");
				});

				bool Result = (ChildProcess.ExitCode == 0 && StdErr.Length == 0);
				return (Result, StdOut, StdErr);
			}
			else
			{
				return (false, "", $"Timed out while trying to run: {CmdPath} {CommandLine}");
			}
		}
	}

	internal class InterceptSave : IVsRunningDocTableEvents3
	{
		private readonly DTE DTE;
		private readonly RunningDocumentTable RunningDocumentTable;
		private readonly P4Commands P4Ops;

		public InterceptSave(DTE InDTE, RunningDocumentTable InRrunningDocumentTable, P4Commands InP4Ops)
		{
			DTE = InDTE;
			RunningDocumentTable = InRrunningDocumentTable;
			P4Ops = InP4Ops;
		}

		public int OnBeforeSave(uint DocumentCookie)
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			RunningDocumentInfo DocumentInfo = RunningDocumentTable.GetDocumentInfo(DocumentCookie);

			string AbsoluteFilePath = DocumentInfo.Moniker;

			P4Ops.OpenForEdit(AbsoluteFilePath, true);

			return VSConstants.S_OK;
		}

		// we are only using OnBeforeSave - but the interface requires us to define the whole interface.
		public int OnAfterFirstDocumentLock(uint docCookie, uint dwRdtLockType, uint dwReadLocksRemaining, uint dwEditLocksRemaining)
		{
			return VSConstants.S_OK;
		}

		public int OnBeforeLastDocumentUnlock(uint docCookie, uint dwRdtLockType, uint dwReadLocksRemaining, uint dwEditLocksRemaining)
		{
			return VSConstants.S_OK;
		}

		public int OnAfterSave(uint docCookie)
		{
			return VSConstants.S_OK;
		}

		public int OnAfterAttributeChange(uint docCookie, uint grfAttribs)
		{
			return VSConstants.S_OK;
		}

		public int OnBeforeDocumentWindowShow(uint docCookie, int fFirstShow, IVsWindowFrame pFrame)
		{
			return VSConstants.S_OK;
		}

		public int OnAfterDocumentWindowHide(uint docCookie, IVsWindowFrame pFrame)
		{
			return VSConstants.S_OK;
		}

		int IVsRunningDocTableEvents3.OnAfterAttributeChangeEx(uint docCookie, uint grfAttribs, IVsHierarchy pHierOld, uint itemidOld,
			string pszMkDocumentOld, IVsHierarchy pHierNew, uint itemidNew, string pszMkDocumentNew)
		{
			return VSConstants.S_OK;
		}

		int IVsRunningDocTableEvents2.OnAfterAttributeChangeEx(uint docCookie, uint grfAttribs, IVsHierarchy pHierOld, uint itemidOld,
			string pszMkDocumentOld, IVsHierarchy pHierNew, uint itemidNew, string pszMkDocumentNew)
		{
			return VSConstants.S_OK;
		}
	}
}
