// Copyright Epic Games, Inc. All Rights Reserved.

using Microsoft.VisualStudio.Shell;
using Microsoft.VisualStudio.Shell.Interop;
using System;
using System.ComponentModel;
using System.Runtime.InteropServices;

namespace UnrealVS
{
	// GUID for this class.
	[Guid(GuidList.UnrealVSPackageManString)]

	class ProfileManager : Component, IProfileManager
	{
		public void SaveSettingsToXml(IVsSettingsWriter writer)
		{
			throw new NotImplementedException();
		}

		public void LoadSettingsFromXml(IVsSettingsReader reader)
		{
			throw new NotImplementedException();
		}

		public void SaveSettingsToStorage()
		{
			throw new NotImplementedException();
		}

		public void LoadSettingsFromStorage()
		{
			throw new NotImplementedException();
		}

		public void ResetSettings()
		{
			throw new NotImplementedException();
		}
	}
}
