// Copyright Epic Games, Inc. All Rights Reserved.

using Microsoft.VisualStudio.Shell;
using System;
using System.Globalization;

namespace UnrealVS
{
	/// <summary>
	/// This attribute registers the package as a solution property parser
	/// </summary>
	[AttributeUsage(AttributeTargets.Class, AllowMultiple = true, Inherited = true)]
	internal sealed class ProvideSolutionPropertiesAttribute : RegistrationAttribute
	{
		private readonly string _propName;

		public ProvideSolutionPropertiesAttribute(string propName)
		{
			_propName = propName;
		}

		public override void Register(RegistrationContext context)
		{
			context.Log.WriteLine(string.Format(CultureInfo.InvariantCulture, "ProvideSolutionProps: ({0} = {1})", context.ComponentType.GUID.ToString("B"), PropName));

			Key childKey = null;

			try
			{
				childKey = context.CreateKey(string.Format(CultureInfo.InvariantCulture, "{0}\\{1}", "SolutionPersistence", PropName));

				childKey.SetValue(string.Empty, context.ComponentType.GUID.ToString("B").ToUpperInvariant());
			}
			finally
			{
				if (childKey != null) childKey.Close();
			}
		}

		public override void Unregister(RegistrationContext context)
		{
			context.RemoveKey(string.Format(CultureInfo.InvariantCulture, "{0}\\{1}", "SolutionPersistence", PropName));
		}

		public string PropName
		{
			get { return _propName; }
		}
	}
}
