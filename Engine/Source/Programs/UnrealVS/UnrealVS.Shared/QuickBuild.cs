// Copyright Epic Games, Inc. All Rights Reserved.

using EnvDTE;
using EnvDTE80;
using Microsoft.VisualStudio;
using Microsoft.VisualStudio.Shell;
using Microsoft.VisualStudio.Shell.Interop;
using System;
using System.Collections.Generic;
using System.ComponentModel.Design;
using System.Linq;
using System.Runtime.InteropServices;

namespace UnrealVS
{
	/// <summary>
	/// Ole Command class for handling the quick build menu items (individual configs inside platform menus)
	/// One of these handles all the items in a platform menu.
	/// </summary>
	class QuickBuildMenuCommand : OleMenuCommand
	{
		/// The function used to select the range of commands handled by this object
		private readonly Predicate<int> OnMatchItem;

		/// <summary>
		/// Create a QuickBuildMenuCommand with a delegate for determining which command numbers it handles
		/// The other params are used to construct the base OleMenuCommand
		/// </summary>
		public QuickBuildMenuCommand(CommandID InDynamicStartCommandId, Predicate<int> InOnMatchItem, EventHandler InOnInvokedDynamicItem, EventHandler InOnBeforeQueryStatusDynamicItem)
			: base(InOnInvokedDynamicItem, null /*changeHandler*/, InOnBeforeQueryStatusDynamicItem, InDynamicStartCommandId)
		{
			OnMatchItem = InOnMatchItem ?? throw new ArgumentNullException("InOnMatchItem");
		}

		/// <summary>
		/// Allows a dynamic item command to match the subsequent items in its list.
		/// Overridden from OleMenuCommand.
		/// Calls OnMatchItem to choose with command numbers are valid.
		/// </summary>
		public override bool DynamicItemMatch(int cmdId)
		{
			// Call the supplied predicate to test whether the given cmdId is a match.
			// If it is, store the command id in MatchedCommandid
			// for use by any BeforeQueryStatus handlers, and then return that it is a match.
			// Otherwise clear any previously stored matched cmdId and return that it is not a match.
			if (OnMatchItem(cmdId))
			{
				MatchedCommandId = cmdId;
				return true;
			}
			MatchedCommandId = 0;
			return false;
		}
	}

	/// <summary>
	/// Class represents a single configuration quick build menu. Wraps QuickBuildMenuCommand.
	/// </summary>
	class ConfigurationMenuContents
	{
		/// The command id of the placeholder command that sets the start of the command range for this menu
		private readonly int DynamicStartCommandId;
		/// The name of the configuration. Matches the config string in the solution configs exactly.
		private readonly string ConfigurationName;

		/// <summary>
		/// Construct a platform menu for a given, named platform, starting at the given command id
		/// </summary>
		/// <param name="Name">The name of the platform</param>
		/// <param name="InDynamicStartCommandId">The start of the command range for this menu</param>
		public ConfigurationMenuContents(string Name, int InDynamicStartCommandId)
		{
			DynamicStartCommandId = InDynamicStartCommandId;
			ConfigurationName = Name;

			// Create the dynamic menu
			var MenuOleCommand = new QuickBuildMenuCommand(
				new CommandID(GuidList.UnrealVSCmdSet, DynamicStartCommandId),
				IsValidDynamicItem,
				OnInvokedDynamicItem,
				OnBeforeQueryStatusDynamicItem
			);
			UnrealVSPackage.Instance.MenuCommandService.AddCommand(MenuOleCommand);
		}

		/// <summary>
		/// The callback function passed to this object's QuickBuildMenuCommand, used to select the range of commands handled by this object
		/// It uses the cached list of solution configs in QuickBuild to determine how many commands should be in the menu.
		/// </summary>
		/// <param name="cmdId">Command id the validate</param>
		/// <returns>True, if the command is valid for this menu</returns>
		private bool IsValidDynamicItem(int cmdId)
		{
			int ConfigCount = QuickBuild.CachedSolutionConfigPlatforms.Length;

			// The match is valid if the command ID is >= the id of our root dynamic start item
			// and the command ID minus the ID of our root dynamic start item
			// is less than or equal to the number of projects in the solution.
			return (cmdId >= DynamicStartCommandId) && ((cmdId - DynamicStartCommandId) < ConfigCount);
		}

		/// <summary>
		/// The invoke handler passed to this object's QuickBuildMenuCommand, called by the base OleMenuCommand when an item is clicked
		/// </summary>
		private void OnInvokedDynamicItem(object Sender, EventArgs args)
		{
			ThreadHelper.ThrowIfNotOnUIThread();
			var MenuCommand = (QuickBuildMenuCommand)Sender;

			// Get the project clicked in the solution explorer by accessing the current selection and converting to a Project if possible.
			if (Utils.GetCurrentlySelectedProject() is Project SelectedProject)
			{
				// Builds the selected project with the clicked platform and config
				Utils.ExecuteProjectBuild(SelectedProject, ConfigurationName, MenuCommand.Text, BatchBuilderToolControl.BuildJob.BuildJobType.Build, null, null);
			}
		}

		/// <summary>
		/// Before-query handler passed to this object's QuickBuildMenuCommand, called by the base OleMenuCommand to update the state of an item.
		/// </summary>
		private void OnBeforeQueryStatusDynamicItem(object sender, EventArgs args)
		{
			var MenuCommand = (QuickBuildMenuCommand)sender;
			MenuCommand.Enabled = QuickBuild.IsActive;
			MenuCommand.Visible = true;

			// Determine the index of the item in the menu
			bool isRootItem = MenuCommand.MatchedCommandId == 0;
			int CommandId = isRootItem ? DynamicStartCommandId : MenuCommand.MatchedCommandId;
			int DynCommandIdx = CommandId - DynamicStartCommandId;

			// Set the text label based on the index and the solution platforms array cached in QuickBuild
			if (DynCommandIdx < QuickBuild.CachedSolutionConfigPlatforms.Length)
			{
				MenuCommand.Text = QuickBuild.CachedSolutionConfigPlatforms[DynCommandIdx];
			}

			// Clear the id now that the query is done
			MenuCommand.MatchedCommandId = 0;
		}
	}

	/// <summary>
	/// The Quick Build menu system
	/// </summary>
	public class QuickBuild
	{
		/** classes */

		/// All the basic data needed to build each platform-specific submenu
		struct SubMenu
		{
			public string Name { get; set; }
			public int SubMenuId { get; set; }
			public int DynamicStartCommandId { get; set; }
		}

		/** constants */

		private const int ProjectQuickBuildMenuID = 0x1410;     // must match the values in the vsct file

		/** fields */

		/// Solution configs and platforms cached once before each time the Quick Build menu opens
		private static string[] SolutionConfigNames = new string[0];
		private static string[] SolutionConfigPlatforms = new string[0];

		/// Hide/shows the while Quick Build menu tree
		private static bool bIsActive = false;

		/// List of submenus and their details - must match the values in the vsct file
		private readonly SubMenu[] SubMenus = new[]
		{
			new SubMenu {Name = "Debug", SubMenuId = 0x1500, DynamicStartCommandId = 0x1530},
			new SubMenu {Name = "Debug Client", SubMenuId = 0x1600, DynamicStartCommandId = 0x1630},
			new SubMenu {Name = "Debug Editor", SubMenuId = 0x1700, DynamicStartCommandId = 0x1730},
			new SubMenu {Name = "Debug Server", SubMenuId = 0x1800, DynamicStartCommandId = 0x1830},
			new SubMenu {Name = "DebugGame", SubMenuId = 0x1900, DynamicStartCommandId = 0x1930},
			new SubMenu {Name = "DebugGame Client", SubMenuId = 0x1A00, DynamicStartCommandId = 0x1A30},
			new SubMenu {Name = "DebugGame Editor", SubMenuId = 0x1B00, DynamicStartCommandId = 0x1B30},
			new SubMenu {Name = "DebugGame Server", SubMenuId = 0x1C00, DynamicStartCommandId = 0x1C30},
			new SubMenu {Name = "Development", SubMenuId = 0x1D00, DynamicStartCommandId = 0x1D30},
			new SubMenu {Name = "Development Client", SubMenuId = 0x1E00, DynamicStartCommandId = 0x1E30},
			new SubMenu {Name = "Development Editor", SubMenuId = 0x1F00, DynamicStartCommandId = 0x1F30},
			new SubMenu {Name = "Development Server", SubMenuId = 0x2000, DynamicStartCommandId = 0x2030},
			new SubMenu {Name = "Shipping", SubMenuId = 0x2100, DynamicStartCommandId = 0x2130},
			new SubMenu {Name = "Shipping Client", SubMenuId = 0x2200, DynamicStartCommandId = 0x2230},
			new SubMenu {Name = "Shipping Editor", SubMenuId = 0x2300, DynamicStartCommandId = 0x2330},
			new SubMenu {Name = "Shipping Server", SubMenuId = 0x2400, DynamicStartCommandId = 0x2430},
			new SubMenu {Name = "Test", SubMenuId = 0x2500, DynamicStartCommandId = 0x2530},
			new SubMenu {Name = "Test Client", SubMenuId = 0x2600, DynamicStartCommandId = 0x2630},
			new SubMenu {Name = "Test Editor", SubMenuId = 0x2700, DynamicStartCommandId = 0x2730},
			new SubMenu {Name = "Test Server", SubMenuId = 0x2800, DynamicStartCommandId = 0x2830},

		};

		/// The main root command of the Quick Build menu hierarchy - used to hide it when not active
		private readonly OleMenuCommand QuickBuildCommand;

		/// <summary>
		/// These represent the items that can be added to the menu that lists the platforms.
		/// Each one is a submenu containing items for each config.
		/// </summary>
		private readonly Dictionary<string, OleMenuCommand> AllConfigurationMenus = new Dictionary<string, OleMenuCommand>();

		/// <summary>
		/// These represent the items shown in the menu that lists the configurations.
		/// It is a subset of AllConfigurationMenus with only the loaded platforms.
		/// Each one is a submenu containing items for each config.
		/// </summary>
		private readonly Dictionary<string, OleMenuCommand> ActiveConfigurationMenus = new Dictionary<string, OleMenuCommand>();

		/// <summary>
		/// These holds the objects that generate the per platform submenus for each configuration.
		/// </summary>
		// ReSharper disable once CollectionNeverQueried.Local - This is only invoked by visual studio to populate the submenus.
		private readonly Dictionary<string, ConfigurationMenuContents> ConfigurationMenusContents = new Dictionary<string, ConfigurationMenuContents>();

		/// VSConstants.UICONTEXT_SolutionBuilding translated into a cookie used to access UI ctxt state
		private readonly uint SolutionBuildingUIContextCookie;

		/** properties */

		public static bool IsActive { get { return bIsActive; } }
		public static string[] CachedSolutionConfigPlatforms { get { return SolutionConfigPlatforms; } }

		/** methods */

		public QuickBuild()
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			// root menu
			QuickBuildCommand = new OleMenuCommand(null, null, OnQuickBuildQuery, new CommandID(GuidList.UnrealVSCmdSet, ProjectQuickBuildMenuID));
			QuickBuildCommand.BeforeQueryStatus += OnQuickBuildQuery;
			UnrealVSPackage.Instance.MenuCommandService.AddCommand(QuickBuildCommand);

			// configuration sub-menus
			foreach (var SubMenu in SubMenus)
			{
				var SubMenuCommand = new OleMenuCommand(null, new CommandID(GuidList.UnrealVSCmdSet, SubMenu.SubMenuId));
				SubMenuCommand.BeforeQueryStatus += OnQuickBuildSubMenuQuery;
				UnrealVSPackage.Instance.MenuCommandService.AddCommand(SubMenuCommand);
				AllConfigurationMenus.Add(SubMenu.Name, SubMenuCommand);
				ConfigurationMenusContents.Add(SubMenu.Name, new ConfigurationMenuContents(SubMenu.Name, SubMenu.DynamicStartCommandId));
			}
			// cache the cookie for UICONTEXT_SolutionBuilding
			UnrealVSPackage.Instance.SelectionManager.GetCmdUIContextCookie(VSConstants.UICONTEXT_SolutionBuilding, out SolutionBuildingUIContextCookie);

			// Initialize the active state based on whether the IDE is building anything
			UpdateActiveState();

			UnrealVSPackage.Instance.OnUIContextChanged += OnUIContextChanged;
			UnrealVSPackage.Instance.OnSolutionOpened += OnSolutionChanged;

			OnSolutionChanged();
		}

		/// <summary>
		/// Updates the bIsActive flag using the UICONTEXT_SolutionBuilding state. Hides the Quick Build feature when the solution is building.
		/// </summary>
		private void UpdateActiveState()
		{
			ThreadHelper.ThrowIfNotOnUIThread();
			UnrealVSPackage.Instance.SelectionManager.IsCmdUIContextActive(SolutionBuildingUIContextCookie, out int bIsBuilding);
			bIsActive = bIsBuilding == 0;
		}

		/// <summary>
		/// Before-query handler passed to the root menu item's OleMenuCommand and called to update the state of the item.
		/// </summary>
		private void OnQuickBuildQuery(object sender, EventArgs e)
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			// Always cache the list of solution build configs when the project menu is opening
			CacheBuildConfigs();
		}

		/// <summary>
		/// Before-query handler passed to the sub-menu item's OleMenuCommand and called to update the state of the item.
		/// </summary>
		private void OnQuickBuildSubMenuQuery(object sender, EventArgs e)
		{
			var SubMenuCommand = (OleMenuCommand)sender;
			SubMenuCommand.Visible = ActiveConfigurationMenus.ContainsValue(SubMenuCommand);
			SubMenuCommand.Enabled = bIsActive;
		}

		/// <summary>
		/// Called when a solution loads. Caches the solution build configs and sets the platform menus' visibility.
		/// Only configs found in the loaded solution's list are shown.
		/// </summary>
		private void OnSolutionChanged()
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			CacheBuildConfigs();

			ActiveConfigurationMenus.Clear();
			Logging.WriteLine("Updating solution menu state, currently active configs are: " + string.Join(", ", SolutionConfigNames));
			foreach (var SubMenu in SubMenus)
			{
				if (SolutionConfigNames.Any(Config => string.Compare(Config, SubMenu.Name, StringComparison.InvariantCultureIgnoreCase) == 0))
				{
					ActiveConfigurationMenus.Add(SubMenu.Name, AllConfigurationMenus[SubMenu.Name]);
				}
			}
		}

		/// <summary>
		/// Caches the solution build configs
		/// </summary>
		private void CacheBuildConfigs()
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			SolutionConfigurations SolutionConfigs = UnrealVSPackage.Instance.DTE.Solution.SolutionBuild.SolutionConfigurations;

			SolutionConfigPlatforms = (from SolutionConfiguration2 Sc in SolutionConfigs select Sc.PlatformName).Distinct().ToArray();

			SolutionConfigNames = (from SolutionConfiguration2 Sc in SolutionConfigs select Sc.Name).Distinct().ToArray();
		}

		/// <summary>
		/// Called when the UI Context changes. If the building state changed, update the active state.
		/// </summary>
		private void OnUIContextChanged(uint CmdUICookie, bool bActive)
		{
			ThreadHelper.ThrowIfNotOnUIThread();
			if (SolutionBuildingUIContextCookie == CmdUICookie)
			{
				UpdateActiveState();
			}
		}
	}
}
