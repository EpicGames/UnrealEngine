// Copyright Epic Games, Inc. All Rights Reserved.

using EnvDTE;
using Microsoft.VisualStudio.Shell;
using Microsoft.VisualStudio.Shell.Interop;
using System;
using System.Collections.Generic;
using System.ComponentModel.Design;
using System.Linq;
using System.Runtime.InteropServices;

namespace UnrealVS
{
	public class StartupProjectSelector
	{
		const int StartupProjectComboId = 0x1050;
		const int StartupProjectComboListId = 0x1060;

		public class StartupProjectListChangedEventArgs : EventArgs
		{
			public Project[] StartupProjects { get; set; }
		}
		public delegate void StartupProjectListChangedEventHandler(object sender, StartupProjectListChangedEventArgs e);
		public event StartupProjectListChangedEventHandler StartupProjectListChanged;

		public bool? CachedHideNonGameStartupProjects { get; set; }

		private class ProjectReference
		{
			public Project Project { get; }
			public string Name { get; }

			public ProjectReference(Project Project)
			{
				ThreadHelper.ThrowIfNotOnUIThread();
				this.Project = Project;
				this.Name = (Project != null) ? Project.Name : "<invalid>";
			}

			public override string ToString() => Name;
		}

		public StartupProjectSelector()
		{
			ThreadHelper.ThrowIfNotOnUIThread();
			_bIsSolutionOpened = UnrealVSPackage.Instance.DTE.Solution.IsOpen;

			// Create the handlers for our commands
			{
				// StartupProjectCombo
				var StartupProjectComboCommandId = new CommandID(GuidList.UnrealVSCmdSet, StartupProjectComboId);
				var StartupProjectComboCommand = new OleMenuCommand(StartupProjectComboHandler, StartupProjectComboCommandId);
				StartupProjectComboCommand.BeforeQueryStatus += (sender, args) => StartupProjectComboCommand.Enabled = _bIsSolutionOpened;
				UnrealVSPackage.Instance.MenuCommandService.AddCommand(StartupProjectComboCommand);

				// StartupProjectComboList
				var StartupProjectComboListCommandId = new CommandID(GuidList.UnrealVSCmdSet, StartupProjectComboListId);
				var StartupProjectComboListCommand = new OleMenuCommand(StartupProjectComboListHandler, StartupProjectComboListCommandId);
				StartupProjectComboListCommand.BeforeQueryStatus += (sender, args) => StartupProjectComboListCommand.Enabled = _bIsSolutionOpened;
				UnrealVSPackage.Instance.MenuCommandService.AddCommand(StartupProjectComboListCommand);

				StartupProjectComboCommands = new[] { StartupProjectComboCommand, StartupProjectComboListCommand };
			}

			// Register for events that we care about
			UnrealVSPackage.Instance.OnStartupProjectChanged += OnStartupProjectChanged;
			UnrealVSPackage.Instance.OnSolutionOpened +=
				delegate
				{
					ThreadHelper.ThrowIfNotOnUIThread();
					Logging.WriteLine("Opened solution " + UnrealVSPackage.Instance.DTE.Solution.FullName);
					_bIsSolutionOpened = true;
					UpdateStartupProjectCombo();
					UpdateStartupProjectList(Utils.GetAllProjectsFromDTE());
				};
			UnrealVSPackage.Instance.OnSolutionClosing +=
				delegate
				{
					Logging.WriteLine("Closing solution");
					_bIsSolutionOpened = false;
					UpdateStartupProjectCombo();
					_CachedStartupProjects.Clear();
					_CachedStartupProjectNames = null;
				};
			UnrealVSPackage.Instance.OnProjectOpened +=
				delegate (Project OpenedProject)
				{
					ThreadHelper.ThrowIfNotOnUIThread();
					if (_bIsSolutionOpened)
					{
						Logging.WriteLine("Opened project node " + OpenedProject.Name);
						UpdateStartupProjectList(OpenedProject);
					}
					else
					{
						Logging.WriteLine("Opened project node " + OpenedProject.Name + " with the solution CLOSED");
					}
				};
			UnrealVSPackage.Instance.OnProjectClosed +=
				delegate (Project ClosedProject)
				{
					ThreadHelper.ThrowIfNotOnUIThread();
					Logging.WriteLine("Closed project node " + ClosedProject.Name);
					RemoveFromStartupProjectList(ClosedProject);
				};
			UnrealVSPackage.Instance.OptionsPage.OnOptionsChanged += OnOptionsChanged;

			UpdateStartupProjectList(Utils.GetAllProjectsFromDTE());
			UpdateStartupProjectCombo();
		}

		public Project[] StartupProjectList
		{
			get
			{
				return (from ProjectRef in _CachedStartupProjects select ProjectRef.Project).ToArray();
			}
		}

		private void OnOptionsChanged(object Sender, EventArgs E)
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			UpdateCachedOptions();

			UpdateStartupProjectList(Utils.GetAllProjectsFromDTE());
		}

		private void UpdateCachedOptions()
		{
			CachedHideNonGameStartupProjects = UnrealVSPackage.Instance.OptionsPage.HideNonGameStartupProjects;
		}

		/// Rebuild/update the list CachedStartupProjectNames whenever something changes
		private void UpdateStartupProjectList(params Project[] DirtyProjects)
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			foreach (Project Project in DirtyProjects)
			{
				if (ProjectPredicate(Project))
				{
					if (!_CachedStartupProjects.Any(ProjRef => ProjRef.Project == Project))
					{
						_CachedStartupProjects.Add(new ProjectReference(Project));
					}
				}
				else
				{
					RemoveFromStartupProjectList(Project);
				}
			}

			SortCachedStartupProjects();

			_CachedStartupProjectNames = null; // Invalidate: list changed
			StartupProjectListChanged?.Invoke(this, new StartupProjectListChangedEventArgs { StartupProjects = StartupProjectList });
		}

		private void SortCachedStartupProjects()
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			// Sort projects by game y/n then alphabetically
			_CachedStartupProjects = (_CachedStartupProjects.OrderBy(ProjectRef => ProjectRef.Name == "UE5" ? 0 : 1)
				.ThenBy(ProjectRef => Utils.IsGameProject(ProjectRef.Project) ? 0 : 1)
				.ThenBy(ProjectRef => ProjectRef.Name)).ToList();
		}

		private bool ProjectPredicate(Project Project)
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			// Initialize the cached options if they haven't been read from UnrealVSOptions yet
			if (!CachedHideNonGameStartupProjects.HasValue)
			{
				UpdateCachedOptions();
			}

			// Optionally, ignore non-game projects
			if (CachedHideNonGameStartupProjects.Value)
			{
				if (!Utils.IsGameProject(Project))
				{
					Logging.WriteLine("StartupProjectSelector: Not listing project " + Project.Name + " because it is not a game");
					return false;
				}
			}

			// Always filter out non-executable projects
			if (!Utils.IsProjectBuildable(Project))
			{
				Logging.WriteLine("StartupProjectSelector: Not listing project " + Project.Name + " because it is not executable");
				return false;
			}

			Logging.WriteLine("StartupProjectSelector: Listing project " + Project.Name);
			return true;
		}

		/// Remove projects from the list CachedStartupProjectNames whenever one unloads
		private void RemoveFromStartupProjectList(Project Project)
		{
			_CachedStartupProjects.RemoveAll(ProjRef => ProjRef.Project == Project);
			_CachedStartupProjectNames = null; // Invalidate: list changed

			StartupProjectListChanged?.Invoke(this, new StartupProjectListChangedEventArgs { StartupProjects = StartupProjectList });
		}

		/// <summary>
		/// Updates the combo box after the project loaded state has changed
		/// </summary>
		private void UpdateStartupProjectCombo()
		{
			ThreadHelper.ThrowIfNotOnUIThread();
			foreach (var Command in StartupProjectComboCommands)
			{
				Command.Enabled = _bIsSolutionOpened;
			}
			_CachedStartupProjectComboText = _bIsSolutionOpened ? MakeStartupProjectComboText() : string.Empty;
		}


		/// <summary>
		/// Called when the startup project has changed to a new project.  We'll refresh our interface.
		/// </summary>
		public void OnStartupProjectChanged(Project NewStartupProject)
		{
			ThreadHelper.ThrowIfNotOnUIThread();
			UpdateStartupProjectCombo();
		}


		/// <summary>
		/// Returns a string to display in the startup project combo box, based on project state
		/// </summary>
		/// <returns>String to display</returns>
		private string MakeStartupProjectComboText()
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			string Text = "";

			// Switch to this project!
			UnrealVSPackage.Instance.SolutionBuildManager.get_StartupProject(out IVsHierarchy ProjectHierarchy);
			if (ProjectHierarchy != null)
			{
				var Project = Utils.HierarchyObjectToProject(ProjectHierarchy);
				if (Project != null)
				{
					Text = Project.Name;
				}
			}

			return Text;
		}


		/// Called by combo control to query the text to display or to apply newly-entered text
		void StartupProjectComboHandler(object Sender, EventArgs Args)
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			var OleArgs = (OleMenuCmdEventArgs)Args;

			// If OutValue is non-zero, then Visual Studio is querying the current combo value to display
			if (OleArgs.OutValue != IntPtr.Zero)
			{
				Marshal.GetNativeVariantForObject(_CachedStartupProjectComboText, OleArgs.OutValue);
			}

			// Has a new input value been entered?
			else if (OleArgs.InValue != null)
			{
				var InputString = OleArgs.InValue.ToString();

				if (string.IsNullOrWhiteSpace(InputString))
				{
					return;
				}

				// Lookup the ProjectReference in CachedStartupProjects to get the full name.
				// Support partial/substring input: try exact match first, then prefix, then contains.
				ProjectReference ProjectRefMatch =
					_CachedStartupProjects.FirstOrDefault(ProjRef => string.Equals(ProjRef.Name, InputString, StringComparison.OrdinalIgnoreCase))
					?? _CachedStartupProjects.FirstOrDefault(ProjRef => ProjRef.Name.StartsWith(InputString, StringComparison.OrdinalIgnoreCase))
					?? _CachedStartupProjects.FirstOrDefault(ProjRef => ProjRef.Name.IndexOf(InputString, StringComparison.OrdinalIgnoreCase) >= 0);

				if (ProjectRefMatch != null && ProjectRefMatch.Project != null)
				{
					// Switch to this project using the standard SDK method set_StartupProject.
					// set_StartupProject fires the proper VS events and marks the solution dirty for .suo persistence.
					var ProjectHierarchy = Utils.ProjectToHierarchyObject(ProjectRefMatch.Project);
					UnrealVSPackage.Instance.SolutionBuildManager.set_StartupProject(ProjectHierarchy);
				}
			}
		}

		/// Called by combo control to populate the drop-down list
		void StartupProjectComboListHandler(object Sender, EventArgs Args)
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			var OleArgs = (OleMenuCmdEventArgs)Args;

			// Rebuild names array only when the project list has changed, not on every dropdown open
			if (_CachedStartupProjectNames == null)
			{
				_CachedStartupProjectNames = (from ProjRef in _CachedStartupProjects select ProjRef.Name).ToArray();
			}
			Marshal.GetNativeVariantForObject(_CachedStartupProjectNames, OleArgs.OutValue);
		}

		/// Combo control and list commands for our startup project selector
		readonly OleMenuCommand[] StartupProjectComboCommands;

		/// Keep track of whether a solution is opened
		bool _bIsSolutionOpened;

		/// Store the list of projects that should be shown in the dropdown list
		List<ProjectReference> _CachedStartupProjects = new List<ProjectReference>();

		/// Cached names array for the combo list handler, null means dirty/needs rebuild
		string[] _CachedStartupProjectNames = null;

		/// Cached display text for the startup project combo, avoids repeated COM calls on every VS poll
		string _CachedStartupProjectComboText = string.Empty;
	}
}
