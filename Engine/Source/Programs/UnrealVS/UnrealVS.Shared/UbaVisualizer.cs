// Copyright Epic Games, Inc. All Rights Reserved.

using EnvDTE80;
using Microsoft.VisualStudio.Shell;
using Microsoft.VisualStudio.Shell.Interop;
using System;
using System.ComponentModel.Design;
using System.Diagnostics;
using System.Diagnostics.CodeAnalysis;
using System.Runtime.InteropServices;
using System.Security;
using System.Threading.Tasks;
using System.Windows;
using System.Windows.Forms;
using System.Windows.Interop;

namespace UnrealVS
{
	delegate IntPtr WndProc(IntPtr hWnd, uint msg, IntPtr wParam, IntPtr lParam);
	class UbaVisualizer
	{
		const int UbaVisualizerButtonID = 0x1338;

		public UbaVisualizer()
		{
			// UbaVisualizerButton
			var CommandID = new CommandID(GuidList.UnrealVSCmdSet, UbaVisualizerButtonID);
			var UbaVisualizerButtonCommand = new MenuCommand(new EventHandler(UbaVisualizerButtonHandler), CommandID);
			UnrealVSPackage.Instance.MenuCommandService.AddCommand(UbaVisualizerButtonCommand);

			UnrealVSPackage.Instance.OnSolutionOpened += () => VisualizerControl?.HandleSolutionChanged();
			UnrealVSPackage.Instance.OnSolutionClosed += () => VisualizerControl?.HandleSolutionChanged();
			UnrealVSPackage.Instance.OnProjectLoaded += (p) => VisualizerControl?.HandleSolutionChanged();
			UnrealVSPackage.Instance.OnProjectUnloading += (p) => VisualizerControl?.HandleSolutionChanged();
			//UnrealVSPackage.Instance.OnDocumentActivated += (d) => VisualizerControl?.HandleDocumentActivated(d);
		}

		/// Called when 'UbaVisualizer' button is clicked
		void UbaVisualizerButtonHandler(object Sender, EventArgs Args)
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			ToolWindowPane ToolWindow = UnrealVSPackage.Instance.FindToolWindow(typeof(UbaVisualizerWindow), 0, true);
			if ((null == ToolWindow) || (null == ToolWindow.Frame))
			{
				throw new NotSupportedException(Resources.ToolWindowCreateError);
			}
			IVsWindowFrame ToolWindowFrame = (IVsWindowFrame)ToolWindow.Frame;
			Microsoft.VisualStudio.ErrorHandler.ThrowOnFailure(ToolWindowFrame.Show());
		}

		UbaVisualizerWindowControl VisualizerControl
		{
			get
			{
				ToolWindowPane ToolWindow = UnrealVSPackage.Instance.FindToolWindow(typeof(UbaVisualizerWindow), 0, true);
				if ((null == ToolWindow) || (null == ToolWindow.Frame))
					return null;
				var UbaVisualizer = (UbaVisualizerWindow)ToolWindow;
				return (UbaVisualizerWindowControl)UbaVisualizer.Content;
			}
		}
	}

	[Guid("b205a03c-566f-4dc3-9e2f-99e2df765f1e")]
	public class UbaVisualizerWindow : ToolWindowPane
	{
		public UbaVisualizerWindow() : base(null)
		{
			this.Caption = "UbaVisualizer";

			this.Content = new UbaVisualizerWindowControl(this);
		}

		[global::System.Diagnostics.CodeAnalysis.SuppressMessageAttribute("", "VSTHRD010")]
		protected override bool PreProcessMessage(ref Message m)
        {
			if (m.Msg == 0x0100)//WM_KEYDOWN)
			{
				if (m.WParam == new IntPtr(0x1B)) // VK_ESCAPE
				{
					//((FileBrowserWindowControl)this.Content).HandleEscape();
					return true;
				}
				else if (m.WParam == new IntPtr(0x70)) // VK_F1
				{
					//((FileBrowserWindowControl)this.Content).HandleF1();
					return true;
				}
				else if (m.WParam == new IntPtr(0x74)) // VK_F5
                {
					//((FileBrowserWindowControl)this.Content).HandleF5();
					return true;
                }
			}
			return base.PreProcessMessage(ref m);
        }
    }

	[SuppressMessage("Microsoft.Usage", "CA2216:DisposableTypesShouldDeclareFinalizer")]
	internal class UbaVisualizerHost : Hwnd​Host
	{
		string VisualizerExe;
		UbaVisualizerWindow Window;
		IntPtr HwndVisualizer;
		WndProc HwndProc;
		IntPtr HwndStatic;

		public UbaVisualizerHost(UbaVisualizerWindow window, string visualizerExe)
		{
			Window = window;
			VisualizerExe = visualizerExe;
			ClipToBounds = false;
		}

		protected override HandleRef BuildWindowCore(HandleRef hwndParent)
		{
			HwndProc = new WndProc(CustomWndProc);

			NativeMethods.WNDCLASSEX wc = new NativeMethods.WNDCLASSEX();
			wc.cbSize = Marshal.SizeOf(typeof(NativeMethods.WNDCLASSEX));
			wc.style = 0x0001 | 0x0002 | 0x0008; // CS_HREDRAW | CS_VREDRAW | CS_DBLCLKS;
			wc.hbrBackground = IntPtr.Zero;// (IntPtr)2; //Black background
			wc.cbClsExtra = 0;
			wc.cbWndExtra = 0;
			wc.hInstance = Marshal.GetHINSTANCE(this.GetType().Module);
			wc.hIcon = IntPtr.Zero;
			wc.hCursor = NativeMethods.LoadCursorW(IntPtr.Zero, (IntPtr)32512); // IDC_ARROW
			wc.lpszMenuName = null;
			wc.lpszClassName = "UbaVisualizerWindow";
			wc.lpfnWndProc = Marshal.GetFunctionPointerForDelegate(HwndProc);
			wc.hIconSm = IntPtr.Zero;
			NativeMethods.RegisterClassExW(ref wc);

			int style = 0x50000000; // WS_CHILD | WS_VISIBLE

			IntPtr hwndHost = NativeMethods.CreateWindowEx(0, "UbaVisualizerWindow", "", style, 0, 0, 1, 1, hwndParent.Handle, IntPtr.Zero, IntPtr.Zero, 0);
			string hwndStr = hwndHost.ToString("X").ToLower();

			ProcessStartInfo startInfo = new ProcessStartInfo(VisualizerExe);
			startInfo.UseShellExecute = false;
			startInfo.Arguments = $"-parent={hwndStr}";
			Process.Start(startInfo);

			return new HandleRef(this, hwndHost);
		}

		protected override void DestroyWindowCore(HandleRef hwnd)
		{
			NativeMethods.DestroyWindow(hwnd.Handle);
		}

		(double, double) GetDpiScale()
		{
			PresentationSource source = PresentationSource.FromVisual(this);
			if (source == null)
			{
				return (1.0, 1.0);
			}
			var m = source.CompositionTarget.TransformToDevice;
			return (m.M11, m.M22);
		}

		protected IntPtr CustomWndProc(IntPtr hwnd, uint msg, IntPtr wParam, IntPtr lParam)
		{
			bool hasVisualizer = false;
			bool shouldResize = msg == 0x0005;

			if (HwndVisualizer != IntPtr.Zero && NativeMethods.IsWindow(HwndVisualizer))
			{
				hasVisualizer = true;

				if (msg == 0x0445) // Special UbaVisualizer message for mouse focus
				{
					//NativeMethods.SetWindowPos(hwnd, (IntPtr)(-1), 0, 0, 0, 0, 0x4000 | 0x0010 | 0x0002 | 0x0001);
					//NativeMethods.SetWindowPos(1hwnd, (IntPtr)(-2), 0, 0, 0, 0, 0x4000 | 0x0010 | 0x0002 | 0x0001);

					//if (!System.Windows.Application.Current.MainWindow.IsActive)
					{
						ThreadHelper.JoinableTaskFactory.Run(() =>
						{
							System.Windows.Application.Current.MainWindow.Topmost = true;
							System.Windows.Application.Current.MainWindow.Topmost = false;
							//System.Windows.Application.Current.MainWindow.Activate();
							//var DTE = UnrealVSPackage.Instance.DTE;
							//DTE.MainWindow.Activate();

							//IVsWindowFrame Frame = (IVsWindowFrame)Window.Frame;
							//Frame.ShowNoActivate();
							//Focus();
							//NativeMethods.SetFocus(HwndVisualizer);
							return Task.CompletedTask;
						});
					}
					//Focus();
					//NativeMethods.SetActiveWindow(hwnd);
				}
				else if (msg == 0x0014) // WM_ERASEBKGND
				{
					return new IntPtr(1);
				}
			}
			else
			{
				if (msg == 0x0444) // Special UbaVisualizer message. Called when visualizer process has set this hwnd as parent
				{
					NativeMethods.DestroyWindow(HwndStatic);
					HwndStatic = IntPtr.Zero;
					HwndVisualizer = lParam;
					hasVisualizer = true;
					shouldResize = true;
				}
				else if (msg == 0x02E0) // WM_DPICHANGED
				{
					shouldResize = true;
				}
				else if (HwndVisualizer != IntPtr.Zero)
				{
					HwndVisualizer = IntPtr.Zero;
					int style = 0x50000000; // WS_CHILD | WS_VISIBLE
					string text = $"Waiting for UbaVisualizer process... (parent hwnd: {hwnd.ToString("X").ToLower()})";
					(double scaleWidth, double scaleHeight) = GetDpiScale();
					HwndStatic = NativeMethods.CreateWindowEx(0, "static", text, style, 0, 0, (int)(ActualWidth * scaleWidth), (int)(ActualHeight * scaleHeight), hwnd, IntPtr.Zero, IntPtr.Zero, 0);
				}
			}

			IntPtr res = NativeMethods.DefWindowProc(hwnd, msg, wParam, lParam);

			if (shouldResize)
			{
				uint flags = 0;
				IntPtr HwndToResize = IntPtr.Zero;

				if (hasVisualizer)
				{
					flags = 0x0040 | 0x4000; // SWP_SHOWWINDOW | SWP_ASYNCWINDOWPOS
					HwndToResize = HwndVisualizer;
				}
				else if (HwndStatic != IntPtr.Zero)
				{
					flags = 0x0040;
					HwndToResize = HwndStatic;
				}

				if (flags != 0)
				{
					(double scaleWidth, double scaleHeight) = GetDpiScale();
					NativeMethods.SetWindowPos(HwndToResize, IntPtr.Zero, 0, 0, (int)(ActualWidth * scaleWidth), (int)(ActualHeight* scaleHeight), flags);
				}
			}
			return res;
		}
	}
}
