<UserControl x:Class="UnrealVS.UbaVisualizerWindowControl"
             xmlns="http://schemas.microsoft.com/winfx/2006/xaml/presentation"
             xmlns:x="http://schemas.microsoft.com/winfx/2006/xaml"
             xmlns:mc="http://schemas.openxmlformats.org/markup-compatibility/2006"
             xmlns:d="http://schemas.microsoft.com/expression/blend/2008"
             xmlns:vsshell="clr-namespace:Microsoft.VisualStudio.Shell;assembly=Microsoft.VisualStudio.Shell.15.0"
             Background="{DynamicResource {x:Static vsshell:VsBrushes.ToolWindowBackgroundKey}}"
             Foreground="{DynamicResource {x:Static vsshell:VsBrushes.WindowTextKey}}"
             mc:Ignorable="d"
             d:DesignHeight="300" d:DesignWidth="500"
             Name="FileBrowser">
	<Border Name="ControlHostElement" />
</UserControl>
