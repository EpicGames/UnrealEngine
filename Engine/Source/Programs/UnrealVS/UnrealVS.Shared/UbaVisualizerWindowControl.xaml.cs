// Copyright Epic Games, Inc. All Rights Reserved.

using Microsoft.VisualStudio.Shell;
using System;
using System.Diagnostics;
using System.IO;
using System.Linq.Expressions;
using System.Windows.Controls;

#pragma warning disable VSTHRD010

namespace UnrealVS
{
	public partial class UbaVisualizerWindowControl : UserControl
	{
		string LastSolutionPath = "Uninitialized";
		string DefaultText = "Starts when opening solution that contains Engine path";
		internal UbaVisualizerWindow Window;

		public UbaVisualizerWindowControl(UbaVisualizerWindow window)
		{
			Window = window;

			InitializeComponent();


			Environment.SetEnvironmentVariable("UBA_OWNER_ID", "vs");
			Environment.SetEnvironmentVariable("UBA_OWNER_PID", Process.GetCurrentProcess().Id.ToString());

			HandleSolutionChanged();
		}

		void SetChild(string visualizerPath, string text)
		{
			var old = ControlHostElement.Child as UbaVisualizerHost;

			if (visualizerPath != null)
			{
				ControlHostElement.Child = new UbaVisualizerHost(Window, visualizerPath);
			}
			else if (ControlHostElement.Child is TextBlock textBlock)
			{
				textBlock.Text = text;
				return;
			}
			else
			{
				textBlock = new TextBlock();
				textBlock.HorizontalAlignment = System.Windows.HorizontalAlignment.Center;
				textBlock.VerticalAlignment = System.Windows.VerticalAlignment.Center;
				textBlock.Text = text;
				ControlHostElement.Child = textBlock;
			}

			if (old != null)
			{
				old.Dispose();
			}
		}

		string SearchForVisualizer(string dir)
		{
			while (true)
			{
				string fullPath = Path.Combine(dir, @"Engine\Binaries\Win64\UnrealBuildAccelerator\x64\UbaVisualizer.exe");
				try
				{
					FileInfo fi = new FileInfo(fullPath);
					if (fi.Exists)
					{
						return fullPath;
					}
				}
				catch
				{
				}
				int lastSlash = dir.LastIndexOf('\\');
				if (lastSlash == -1)
				{
					break;
				}
				dir = dir.Substring(0, lastSlash);
			}
			return null;
		}

		public void HandleSolutionChanged()
		{
			string path = UnrealVSPackage.Instance.SolutionFilepath;
			if (path == null)
			{
				path = string.Empty;
			}

			if (LastSolutionPath == path)
			{
				return;
			}
			LastSolutionPath = path;

			if (path == string.Empty)
			{
				SetChild(null, DefaultText);
				return;
			}

			string fullPath = SearchForVisualizer(Path.GetDirectoryName(path));
			if (fullPath == null)
			{
				foreach (string projectPath in UnrealVSPackage.Instance.GetLoadedProjectPaths())
				{
					if (!projectPath.Contains("\\Engine\\"))
					{
						continue;
					}
					fullPath = SearchForVisualizer(Path.GetDirectoryName(projectPath));
					break;
				}	
			}

			if (fullPath == null)
			{
				SetChild(null, DefaultText);
				return;
			}

			var versionInfo = FileVersionInfo.GetVersionInfo(fullPath);
			var pv = versionInfo.ProductVersion;
			if (string.IsNullOrEmpty(pv))
			{
				pv = "Uba_v0.x.x";
			}

			if (pv.Length <= 5 || !pv.StartsWith("Uba_v") || pv[5] == '0')
			{
				SetChild(null, $"UbaVisualizer.exe found is too old to be embedded in visual studio ({pv})");
				return;
			}


			SetChild(fullPath, "");
		}
	}

}

#pragma warning restore VSTHRD010
