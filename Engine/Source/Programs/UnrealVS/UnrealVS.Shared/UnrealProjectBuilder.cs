// Copyright Epic Games, Inc. All Rights Reserved.

using EnvDTE;
using Microsoft.VisualStudio.Shell;
using Microsoft.VisualStudio.Shell.Interop;
using Microsoft.VisualStudio.VCProjectEngine;
using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Linq;
using System.Text.RegularExpressions;
using System.Windows.Forms;

namespace UnrealVS
{
	class UnrealProjectBuilder : IDisposable
	{
		System.Diagnostics.Process ChildProcess;
		private string FileToCompileOriginalExt = "";

		public UnrealProjectBuilder()
		{
		}

		public void Dispose()
		{
			KillChildProcess();
		}

		/// <summary>
		/// Removes any -Target="..." arguments after the first one from the provided string.
		/// </summary>
		/// <param name="BuildCommandLine">Set of arguments to reduce</param>
		/// <returns>New, reduced command line</returns>
		private static string RemoveExtraTargets(string BuildCommandLine)
		{
			if (string.IsNullOrEmpty(BuildCommandLine))
			{
				return BuildCommandLine;
			}

			// Match: -Target="..."
			MatchCollection TargetMatches = Regex.Matches(BuildCommandLine, @"\s*-Target=""[^""]*""", RegexOptions.IgnoreCase);
			if (TargetMatches.Count <= 1)
			{
				// No multi-target, return as-is
				return BuildCommandLine;
			}

			string NewCommandLine = BuildCommandLine;

			// Remove every -Target="" after the first one. Iterate in reverse order to preserve the match indices.
			foreach (Match TargetMatch in TargetMatches.Cast<Match>().Skip(1).Reverse())
			{
				NewCommandLine = NewCommandLine.Remove(TargetMatch.Index, TargetMatch.Length);
			}

			return NewCommandLine;
		}

		public bool BuildProject(Project TargetProject, string SpecificTarget, List<string> ExtraArguments, bool bProfiling, bool bSingleTarget = false)
		{
			ThreadHelper.ThrowIfNotOnUIThread();
			DTE DTE = UnrealVSPackage.Instance.DTE;

			if (!(TargetProject.Object is VCProject VCProject))
			{
				Logging.WriteLine("UnrealProjectBuilder: VCProject not found");
				return false;
			}

			// Activate the output window
			Window Window = DTE.Windows.Item(EnvDTE.Constants.vsWindowKindOutput);
			Window.Activate();

			// Find or create the 'Build' window
			if (!(UnrealVSPackage.Instance.GetOutputPane() is IVsOutputWindowPane BuildOutputPane))
			{
				Logging.WriteLine("UnrealProjectBuilder: Build Output Pane not found");
				return false;
			}

			// If there's already a build in progress, offer to cancel it
			if (ChildProcess != null && !ChildProcess.HasExited)
			{
				if (MessageBox.Show("Cancel current compile?", "Compile in progress", MessageBoxButtons.YesNo) == DialogResult.Yes)
				{
					KillChildProcess();
					BuildOutputPane.OutputStringThreadSafe($"1>  Build cancelled.{Environment.NewLine}");
				}
				return true;
			}

			Configuration ActiveConfiguration = TargetProject.ConfigurationManager.ActiveConfiguration;
			string ActiveConfigurationName = $"{ActiveConfiguration.ConfigurationName}|{ActiveConfiguration.PlatformName}";
			if (!((VCProject.Configurations as IVCCollection).Item(ActiveConfigurationName) is VCConfiguration ActiveVCConfiguration))
			{
				Logging.WriteLine("UnrealProjectBuilder: Project ActiveConfiguration not found");
				return false;
			}

			// Get the NMake settings for this configuration
			if (!((ActiveVCConfiguration.Tools as IVCCollection).Item("VCNMakeTool") is VCNMakeTool ActiveNMakeTool))
			{
				MessageBox.Show($"No NMakeTool set for Project {VCProject.Name} set for single-file compile.", "UnrealVS - NMakeTool not set", MessageBoxButtons.OK);
				return false;
			}

			// Save all the open documents
			DTE.ExecuteCommand("File.SaveAll");

			// If there's already a build in progress, don't let another one start
			if (DTE.Solution.SolutionBuild.BuildState == vsBuildState.vsBuildStateInProgress)
			{
				if (MessageBox.Show("Cancel current compile?", "Compile in progress", MessageBoxButtons.YesNo) == DialogResult.Yes)
				{
					DTE.ExecuteCommand("Build.Cancel");
				}
				return true;
			}

			// Make sure any existing build is stopped
			KillChildProcess();

			if (DTE.ActiveDocument != null)
			{
				FileToCompileOriginalExt = Path.GetExtension(DTE.ActiveDocument.FullName);
			}

			List<string> UBTArguments = new List<string>
			{
				"-WorkingDir=\"$(MSBuildProjectDirectory)\"",
			};

			UBTArguments.AddRange(ExtraArguments);

			// Set up the output pane
			BuildOutputPane.Activate();
			BuildOutputPane.Clear();

			// Set up event handlers 
			DTE.Events.BuildEvents.OnBuildBegin += BuildEvents_OnBuildBegin;

			// Create a delegate for handling output messages
			List<string> PreprocessedFiles = new List<string>();
			List<string> AssemblyFiles = new List<string>();
			void OutputHandler(object Sender, DataReceivedEventArgs Args)
			{
				if (Args.Data != null)
				{
					if (Args.Data.Contains("PreProcessPath:"))
					{
						PreprocessedFiles.Add(Args.Data.Replace("PreProcessPath:", "").Trim());
					}
					else if (Args.Data.Contains("AssemblyPath:"))
					{
						AssemblyFiles.Add(Args.Data.Replace("AssemblyPath:", "").Trim());
					}
					else
					{
						BuildOutputPane.OutputStringThreadSafe($"1>  {Args.Data}{Environment.NewLine}");
					}
				}
			}

			if (bProfiling && !CompileScoreHelper.StartTrace())
			{
				return false;
			}

			BuildOutputPane.OutputStringThreadSafe($"1>------ Build started: Project: {TargetProject.Name}, Configuration: {ActiveConfiguration.ConfigurationName} {ActiveConfiguration.PlatformName} ------{Environment.NewLine}");
			if (!string.IsNullOrEmpty(SpecificTarget))
			{
				BuildOutputPane.OutputStringThreadSafe($"1>  Compiling {SpecificTarget}{Environment.NewLine}");
			}

			string NMakeBuildCommandLine = ActiveNMakeTool.BuildCommandLine;
			if (bSingleTarget)
			{
				NMakeBuildCommandLine = RemoveExtraTargets(NMakeBuildCommandLine);
			}

			string SolutionDir = Path.GetDirectoryName(UnrealVSPackage.Instance.SolutionFilepath).TrimEnd(Path.DirectorySeparatorChar) + Path.DirectorySeparatorChar;
			// Get the build command line and escape any environment variables that we use
			string BuildCommandLine = ActiveVCConfiguration.Evaluate(NMakeBuildCommandLine);
			string UBTArgument = ActiveVCConfiguration.Evaluate(string.Join(" ", UBTArguments));
			string WorkingDirectory = ActiveVCConfiguration.Evaluate("$(MSBuildProjectDirectory)");

			// Spawn the new process
			ChildProcess = new System.Diagnostics.Process();
			ChildProcess.StartInfo.FileName = Path.Combine(Environment.SystemDirectory, "cmd.exe");
			ChildProcess.StartInfo.Arguments = $"/C \"{BuildCommandLine} {UBTArgument}\"";
			ChildProcess.StartInfo.WorkingDirectory = WorkingDirectory;
			ChildProcess.StartInfo.UseShellExecute = false;
			ChildProcess.StartInfo.RedirectStandardOutput = true;
			ChildProcess.StartInfo.RedirectStandardError = true;
			ChildProcess.StartInfo.CreateNoWindow = true;
			ChildProcess.OutputDataReceived += OutputHandler;
			ChildProcess.ErrorDataReceived += OutputHandler;
			{
				// add an event handler to respond to the exit of the request
				// and open the generated file if it exists.
				ChildProcess.EnableRaisingEvents = true;
				ChildProcess.Exited += new EventHandler((s, e) => UbtProcessExitHandler(PreprocessedFiles, AssemblyFiles, bProfiling));
			}

			ChildProcess.Start();
			ChildProcess.BeginOutputReadLine();
			ChildProcess.BeginErrorReadLine();

			return true;
		}

		public bool BuildSingleFile(Project TargetProject, string FileToCompile, List<string> InExtraArguments, bool bProfiling)
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			List<string> ExtraArguments = new List<string>()
			{
				$"-SingleFile=\"{FileToCompile}\""
			};
			if (InExtraArguments != null && InExtraArguments.Count > 0)
			{
				ExtraArguments.AddRange(InExtraArguments);
			}
			return BuildProject(TargetProject, FileToCompile, ExtraArguments, bProfiling: bProfiling);
		}

		public bool BuildModule(Project TargetProject, string ModuleName, List<string> InExtraArguments, bool bProfiling)
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			List<string> ExtraArguments = new List<string>()
			{
				$"-Module=\"{ModuleName}\""
			};
			if (InExtraArguments != null && InExtraArguments.Count > 0)
			{
				ExtraArguments.AddRange(InExtraArguments);
			}
			return BuildProject(TargetProject, ModuleName, ExtraArguments, bProfiling: bProfiling, bSingleTarget: true);
		}

		void KillChildProcess()
		{
			if (ChildProcess != null)
			{
				if (!ChildProcess.HasExited)
				{
					ChildProcess.Kill();
					ChildProcess.WaitForExit();
				}
				ChildProcess.Dispose();
				ChildProcess = null;
			}
		}

		private void UbtProcessExitHandler(IEnumerable<string> PreprocessedFiles, IEnumerable<string> AssemblyFiles, bool bProfiling)
		{
			if (bProfiling)
			{
				ThreadHelper.JoinableTaskFactory.Run(async () =>
				{
					await ThreadHelper.JoinableTaskFactory.SwitchToMainThreadAsync();
					CompileScoreHelper.StopTrace();
				});
			}

			// not all compile actions support pre-process - check it exists
			foreach (string PreprocessedFile in PreprocessedFiles)
			{
				ThreadHelper.JoinableTaskFactory.Run(async () =>
				{
					await ThreadHelper.JoinableTaskFactory.SwitchToMainThreadAsync();
					OpenPreprocessedFile(PreprocessedFile);
				});
			}
			foreach (string AssemblyFile in AssemblyFiles)
			{
				ThreadHelper.JoinableTaskFactory.Run(async () =>
				{
					await ThreadHelper.JoinableTaskFactory.SwitchToMainThreadAsync();
					OpenAssemblyFile(AssemblyFile);
				});
			}
		}

		private void OpenPreprocessedFile(string PPFullPath)
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			if (File.Exists(PPFullPath))
			{
				// if the file exists, rename it to isolate the file and have its extension be the original to maintain syntax highlighting
				string Dir = Path.GetDirectoryName(PPFullPath);
				string FileName = Path.GetFileNameWithoutExtension(Path.GetFileNameWithoutExtension(PPFullPath)) + "_preprocessed";

				string RenamedFile = Path.Combine(Dir, FileName) + FileToCompileOriginalExt;

				File.Copy(PPFullPath, RenamedFile, true /*overwrite*/);

				UnrealVSPackage.Instance.DTE.ExecuteCommand("File.OpenFile", $"\"{RenamedFile}\"");
			}
		}

		private void OpenAssemblyFile(string AsmFullPath)
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			if (File.Exists(AsmFullPath))
			{
				UnrealVSPackage.Instance.DTE.ExecuteCommand("File.OpenFile", $"\"{AsmFullPath}\"");
			}
		}

		private void BuildEvents_OnBuildBegin(vsBuildScope Scope, vsBuildAction Action)
		{
			KillChildProcess();
		}
	}
}
