<?xml version="1.0" encoding="utf-8"?>
<Project xmlns="http://schemas.microsoft.com/developer/msbuild/2003">
  <PropertyGroup>
    <MSBuildAllProjects Condition="'$(MSBuildVersion)' == '' Or '$(MSBuildVersion)' &lt; '16.0'">$(MSBuildAllProjects);$(MSBuildThisFileFullPath)</MSBuildAllProjects>
    <HasSharedItems>true</HasSharedItems>
    <SharedGUID>eba9d441-80f1-4a49-87f0-4e2ec713a4f2</SharedGUID>
  </PropertyGroup>
  <PropertyGroup Label="Configuration">
    <Import_RootNamespace>UnrealVS</Import_RootNamespace>
  </PropertyGroup>
  <ItemGroup>
    <Compile Include="$(MSBuildThisFileDirectory)BatchBuilder.cs" />
    <Compile Include="$(MSBuildThisFileDirectory)BatchBuilderToolControl.xaml.cs">
      <DependentUpon>BatchBuilderToolControl.xaml</DependentUpon>
    </Compile>
    <Compile Include="$(MSBuildThisFileDirectory)BatchBuilderToolWindow.cs" />
    <Compile Include="$(MSBuildThisFileDirectory)BuildAndProfile.cs" />
    <Compile Include="$(MSBuildThisFileDirectory)BuildStartupProject.cs" />
    <Compile Include="$(MSBuildThisFileDirectory)BuildStatusIcon.xaml.cs">
      <DependentUpon>BuildStatusIcon.xaml</DependentUpon>
    </Compile>
    <Compile Include="$(MSBuildThisFileDirectory)CommandLineEditor.cs" />
    <Compile Include="$(MSBuildThisFileDirectory)CompileScoreHelper.cs" />
    <Compile Include="$(MSBuildThisFileDirectory)CompileSingleFile.cs" />
    <Compile Include="$(MSBuildThisFileDirectory)CopyIncludePath.cs" />
    <Compile Include="$(MSBuildThisFileDirectory)CycleRelatedFiles.cs" />
    <Compile Include="$(MSBuildThisFileDirectory)FileBrowser.cs" />
    <Compile Include="$(MSBuildThisFileDirectory)FileBrowserWindowControl.xaml.cs">
      <DependentUpon>FileBrowserWindowControl.xaml</DependentUpon>
    </Compile>
    <Compile Include="$(MSBuildThisFileDirectory)UbaVisualizer.cs" />
    <Compile Include="$(MSBuildThisFileDirectory)UbaVisualizerWindowControl.xaml.cs">
      <DependentUpon>UbaVisualizerWindowControl.xaml</DependentUpon>
    </Compile>
    <Compile Include="$(MSBuildThisFileDirectory)GenerateProjectFiles.cs" />
    <Compile Include="$(MSBuildThisFileDirectory)GlobalSuppressions.cs" />
    <Compile Include="$(MSBuildThisFileDirectory)Guids.cs" />
    <Compile Include="$(MSBuildThisFileDirectory)Logging.cs" />
    <Compile Include="$(MSBuildThisFileDirectory)P4Commands.cs" />
    <Compile Include="$(MSBuildThisFileDirectory)ProfileManager.cs">
      <SubType>Component</SubType>
    </Compile>
    <Compile Include="$(MSBuildThisFileDirectory)ProvideSolutionProperties.cs" />
    <Compile Include="$(MSBuildThisFileDirectory)QuickBuild.cs" />
    <Compile Include="$(MSBuildThisFileDirectory)StartupProjectSelector.cs" />
    <Compile Include="$(MSBuildThisFileDirectory)UnrealProjectBuilder.cs" />
    <Compile Include="$(MSBuildThisFileDirectory)UnrealVSOptions.cs">
      <SubType>Component</SubType>
    </Compile>
    <Compile Include="$(MSBuildThisFileDirectory)UnrealVSPackage.cs" />
    <Compile Include="$(MSBuildThisFileDirectory)Utils.cs" />
  </ItemGroup>
  <ItemGroup>
    <Page Include="$(MSBuildThisFileDirectory)BatchBuilderToolControl.xaml">
      <SubType>Designer</SubType>
      <Generator>MSBuild:Compile</Generator>
    </Page>
    <Page Include="$(MSBuildThisFileDirectory)BuildStatusIcon.xaml">
      <SubType>Designer</SubType>
      <Generator>MSBuild:Compile</Generator>
    </Page>
    <Page Include="$(MSBuildThisFileDirectory)FileBrowserWindowControl.xaml">
      <SubType>Designer</SubType>
      <Generator>MSBuild:Compile</Generator>
    </Page>
    <Page Include="$(MSBuildThisFileDirectory)UbaVisualizerWindowControl.xaml">
      <SubType>Designer</SubType>
      <Generator>MSBuild:Compile</Generator>
    </Page>
  </ItemGroup>
</Project>