// Copyright Epic Games, Inc. All Rights Reserved.

using Microsoft.VisualStudio.Shell;
using System;
using System.ComponentModel;
using System.Runtime.InteropServices;

namespace UnrealVS
{
	[Guid(GuidList.UnrealVSOptionsString)]
	public class UnrealVsOptions : DialogPage
	{
		public event EventHandler OnOptionsChanged;

		[Category("General")]
		[DisplayName("Hide Non-Game Startup Projects")]
		[Description("Shows only game projects in the startup project and batch-builder lists")]
		public bool HideNonGameStartupProjects { get; set; }

		[Category("Unreal.P4")]
		[DisplayName("Enable Auto Checkout on Save")]
		[Description("Uses P4 ini / environment settings to automatically checkout files on save, use 'RunUAT P4WriteConfig' to initialize those settings")]
		public bool AllowUnrealVSCheckoutOnEdit { get; set; }

		[Category("Unreal.P4")]
		[DisplayName("Override VS Compare Options")]
		[Description("Unreal VS will override built in diff settings to the ideal for code, does not alter P4")]
		public bool AllowUnrealVSOverrideDiffSettings { get; set; }

		[Category("Unreal.P4")]
		[DisplayName("Use P4V Diff")]
		[Description("Uses P4V to diff files")]
		public bool UseP4VDiff { get; set; }

		[Category("Unreal.P4")]
		[DisplayName("Allow Perforce operations")]
		[Description("Uses P4 ini / environment settings to call P4 functionality, use 'RunUAT P4WriteConfig' to initialize those settings")]
		public bool AllowUnrealVSP4 { get; set; }

		[Category("Unreal.P4")]
		[DisplayName("Allow Async Checkout")]
		[Description("Switches checkout (on edit or manual) to an async method, note if the p4 operation fails the local file will still be writable")]
		public bool AllowAsyncP4Checkout { get; set; }

		[Category("Unreal.P4")]
		[DisplayName("Force P4 Output Window to Active")]
		[Description("When set, normal P4 operations will force the output window to the surface")]
		public bool ForceOutputWindow { get; set; }

		[Category("Unreal.P4")]
		[DisplayName("Allow Reconcile to Mark for Edit")]
		[Description("Switches Reconcile from preview mode to marking files for edit in the default CL")]
		public bool AllowReconcileToMarkForEdit { get; set; }

		[Category("Unreal.P4")]
		[DisplayName("Reconcile Extensions")]
		[Description("Reconcile will attempt to resolve these extensions (including wildcards)")]
		public string ReconcileExtensions { get; set; } = "c*;h*;ini;uproject;uplugin";

		[Category("General")]
		[DisplayName("Prefix UE5 Solution Name with Folder")]
		[Description("Will include name of folder that UE5.sln is located in to Visual Studio window title.")]
		public bool IncludeFolderInUE5SolutionName { get; set; } = true;

		[Category("File Browser")]
		[DisplayName("Keep Open After Selection")]
		[Description("When enabled, the File Browser window stays open after opening a file instead of closing automatically.")]
		public bool FileBrowserKeepOpenAfterSelection { get; set; } = true;

		protected override void OnApply(PageApplyEventArgs e)
		{
			base.OnApply(e);

			if (e.ApplyBehavior == ApplyKind.Apply && OnOptionsChanged != null)
			{
				OnOptionsChanged(this, EventArgs.Empty);
			}
		}
	}


}
