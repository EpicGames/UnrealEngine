// Copyright Epic Games, Inc. All Rights Reserved.

using EnvDTE;
using EnvDTE80;
using Microsoft.VisualStudio;
using Microsoft.VisualStudio.OLE.Interop;
using Microsoft.VisualStudio.Shell;
using Microsoft.VisualStudio.Shell.Interop;
using System;
using System.Collections.Generic;
using System.ComponentModel.Design;
using System.Diagnostics;
using System.IO;
using System.Linq;
using System.Runtime.InteropServices;
using System.Threading;
using Thread = System.Threading.Thread;
using Microsoft.VisualStudio.ComponentModelHost;
using Microsoft.VisualStudio.Text.Editor;
using Microsoft;
using System.Text;

namespace UnrealVS
{
	public class UnrealSolutionProperties
	{
		public UnrealSolutionProperties(string[] InAvailablePlatforms)
		{
			AvailablePlatforms = InAvailablePlatforms;
		}

		public string[] AvailablePlatforms { get; }
	}

	// This attribute tells the PkgDef creation utility (CreatePkgDef.exe) that this class is a VS package.
	[PackageRegistration(UseManagedResourcesOnly = true, AllowsBackgroundLoading = true)]

	// This attribute is used to register the informations needed to show the this package in the Help/About dialog of Visual Studio.
	[InstalledProductRegistration("#110", "#112", VersionString, IconResourceID = 400)]

	// Adds data for our interface elements defined in UnrealVS.vsct.  The name "Menus.ctmenu" here is arbitrary, but must
	// match the ItemGroup inside the UnrealVS.csproj MSBuild file (hand-typed.)
	[ProvideMenuResource("Menus.ctmenu", 1)]

	// Force the package to load whenever a solution exists
	//[ProvideAutoLoad(VSConstants.UICONTEXT.SolutionExistsAndFullyLoaded_string, PackageAutoLoadFlags.BackgroundLoad)]
	[ProvideAutoLoad(UIContextGuids80.SolutionExists, PackageAutoLoadFlags.BackgroundLoad)]

	// Register the settings implementing class as providing support to UnrealVSPackage.
	//[ProvideProfile(typeof (ProfileManager), "UnrealVS", "UnrealVSPackage", 110, 113, false)]

	// GUID for this class.  This needs to match in quite a few places in the project.
	[Guid(GuidList.UnrealVSPackageString)]

	// This attribute registers a tool window exposed by this package.
	[ProvideToolWindow(typeof(BatchBuilderToolWindow))]

	// File browser
	[ProvideToolWindow(typeof(FileBrowserWindow))]

	// UbaVisualizer
	[ProvideToolWindow(typeof(UbaVisualizerWindow))]
	[ProvideToolWindowVisibility(typeof(UbaVisualizerWindow), VSConstants.UICONTEXT.SolutionExists_string)]

	// This attribute registers an options page for the package.
	[ProvideOptionPage(typeof(UnrealVsOptions), ExtensionName, "General", 101, 102, true)]
	[ProvideSolutionProperties(GuidList.UnrealVSPackageString)]

	/// <summary>
	/// UnrealVSPackage implements Package abstract class.  This is the main class that is registered
	/// with Visual Studio shell and serves as the entry point into our extension
	/// </summary>
	public sealed class UnrealVSPackage :
		AsyncPackage,           // We inherit from AsyncPackage which allows us to become a "plugin" within the Visual Studio shell
		IVsSolutionEvents,      // This interface allows us to register to be notified of events such as opening a project
		IVsUpdateSolutionEvents,// Allows us to register to be notified of events such as active config changes
		IVsSelectionEvents,     // Allows us to be notified when the startup project has changed to a different project
		IVsHierarchyEvents,     // Allows us to be notified when a hierarchy (the startup project) has had properties changed
		IVsPersistSolutionProps, // Allows us to read props added to the solution to determine if the solution is a unreal solution
		IVsDebuggerEvents,
		IDisposable
	{
		/** Constants */

		private const string VersionString = "v1.87";
		private const string UnrealSolutionFileNamePrefix = "UE";
		private const string ExtensionName = "UnrealVS";
		private const string CommandLineOptionKey = ExtensionName + "CommandLineMRU";
		private const string BatchBuildSetsOptionKey = ExtensionName + "BatchBuildSetsV002";
		private readonly TimeSpan TickPeriod = TimeSpan.FromSeconds(1.0);

		/** Events */

		/// Called when a new startup project is set in Visual Studio
		public delegate void OnStartupProjectChangedDelegate(Project NewStartupProject);
		public event OnStartupProjectChangedDelegate OnStartupProjectChanged;

		/// Called when a project is opened or created in Visual Studio
		public delegate void OnProjectOpenedDelegate(Project OpenedProject);
		public event OnProjectOpenedDelegate OnProjectOpened;

		/// Called right before a project is closed
		public delegate void OnProjectClosedDelegate(Project ClosedProject);
		public event OnProjectClosedDelegate OnProjectClosed;

		/// Called when a project is loaded in Visual Studio
		public delegate void OnProjectLoadedDelegate(Project LoadedProject);
		public event OnProjectLoadedDelegate OnProjectLoaded;

		/// Called right before a project is unloaded in Visual Studio
		public delegate void OnProjectUnloadingDelegate(Project UnloadedProject);
		public event OnProjectUnloadingDelegate OnProjectUnloading;

		/// Called when the startup project is edited in Visual Studio
		public delegate void OnStartupProjectPropertyChangedDelegate(UInt32 itemid, Int32 propid, UInt32 flags);
		public event OnStartupProjectPropertyChangedDelegate OnStartupProjectPropertyChanged;

		/// Called right after a solution is opened
		public delegate void OnSolutionOpenedDelegate();
		public event OnSolutionOpenedDelegate OnSolutionOpened;

		/// Called right before/after a solution is closed
		public delegate void OnSolutionClosedDelegate();
		public event OnSolutionClosedDelegate OnSolutionClosing;
		public event OnSolutionClosedDelegate OnSolutionClosed;

		/// Called when the active project config changes for any project
		public delegate void OnStartupProjectConfigChangedDelegate(Project Project);
		public event OnStartupProjectConfigChangedDelegate OnStartupProjectConfigChanged;

		/// Called when a build/update action begins
		public delegate void OnBuildBeginDelegate(out int Cancel);
		public event OnBuildBeginDelegate OnBuildBegin;

		/// Called when a build/update action completes
		public delegate void OnBuildDoneDelegate(bool bSucceeded, bool bModified, bool bWasCancelled);
		public event OnBuildDoneDelegate OnBuildDone;

		/// Called when the UIContext changes
		public delegate void OnUIContextChangedDelegate(uint CmdUICookie, bool bActive);
		public event OnUIContextChangedDelegate OnUIContextChanged;

		public delegate void OnDocumentActivatedDelegate(Document Document);
		public event OnDocumentActivatedDelegate OnDocumentActivated;

		/** Public Fields & Properties */

		/// Returns singleton instance of UnrealVSPackage
		public static UnrealVSPackage Instance
		{
			get { return PrivateInstance; }
		}

		/// Visual Studio menu command service
		public IMenuCommandService MenuCommandService { get; private set; }

		/// Visual Studio solution build manager interface.  This is used to change the active startup
		/// Project, among other things.  We expose public access to the solution build manager through
		/// our singleton instance
		public IVsSolutionBuildManager2 SolutionBuildManager { get; private set; }

		/// Visual Studio solution build manager interface.  This is used to change the active startup
		/// Project, among other things.  We expose public access to the solution build manager through
		/// our singleton instance
		public IVsDebugger Debugger { get; private set; }

		/// Visual Studio solution "manager" interface.  We register with this to receive events
		/// about projects being added and such.  This needs to be cleaned up at shutdown.
		public IVsSolution2 SolutionManager { get; private set; }

		/// Our startup project selector component
		public StartupProjectSelector StartupProjectSelector { get; private set; }

		/// Our quick build component
		public QuickBuild QuickBuilder { get; private set; }

		/// Visual Studio shell selection manager interface.  Used to receive notifications about
		/// startup projects changes, among other things.
		public IVsMonitorSelection SelectionManager { get; private set; }

		/// Variable keeps track of whether a supported Unreal solution is loaded
		public bool IsUESolutionLoaded
		{
			get { return _IsUESolutionLoaded.GetValueOrDefault(false); }
			private set { _IsUESolutionLoaded = value; }
		}

		/// Variable keeps track of the loaded solution
		private string _SolutionFilepath;
		public string SolutionFilepath { get { return _SolutionFilepath; } }

		public UnrealSolutionProperties UnrealSolutionProperties { get; private set; }

		/** Methods */

		/// <summary>
		/// Package constructor.  The package is being created but Visual Studio isn't fully initialized yet, so
		/// it's NOT SAFE to call into Visual Studio services from here.  Do that in Initialize() instead.
		/// </summary>
		public UnrealVSPackage()
		{
			// Register this key string so the package can save command line data to solution options files.
			// See OnLoadOptions() & OnSaveOptions()
			AddOptionKey(CommandLineOptionKey);
			AddOptionKey(BatchBuildSetsOptionKey);

			// Setup singleton instance
			PrivateInstance = this;

			Logging.Initialize(ExtensionName, VersionString);
			Logging.WriteLine("Loading UnrealVS extension package...");
		}

		/// <summary>
		/// Initializes the package right after it's been "sited" into the fully-initialized Visual Studio IDE.
		/// </summary>
		protected override async System.Threading.Tasks.Task InitializeAsync(CancellationToken cancellationToken, IProgress<ServiceProgressData> progress)
		{
			Logging.WriteLine("Initializing UnrealVS extension...");

			await JoinableTaskFactory.SwitchToMainThreadAsync(cancellationToken);

			// Grab the MenuCommandService
			MenuCommandService = await GetServiceAsync(typeof(IMenuCommandService)) as OleMenuCommandService;

			// Get access to Visual Studio's DTE object.  This object has various hooks into the Visual Studio
			// shell that are useful for writing extensions.
			DTE = await GetServiceAsync(typeof(DTE)) as DTE;
			Assumes.Present(DTE);
			Logging.WriteLine("DTE version " + DTE.Version);

			// We previously deferred getting the DTE2 object but there are some cases where it's not running yet, so try here
			_DTE2 = await GetServiceAsync(typeof(SDTE)) as DTE2;
			Assumes.Present(_DTE2);
			Assumes.True(_DTE2.DTE == DTE);

			//TextManager = await GetServiceAsync(typeof(VsTextManagerClass)) as IVsTextManager3;

			var componentModel = (IComponentModel)GetGlobalService(typeof(SComponentModel));
			EditorOptionsFactory = componentModel.GetService<IEditorOptionsFactoryService>();

			// Get selection manager and register to receive events
			SelectionManager =
				await GetServiceAsync(typeof(SVsShellMonitorSelection)) as IVsMonitorSelection;
			Assumes.Present(SelectionManager);
			SelectionManager.AdviseSelectionEvents(this, out SelectionEventsHandle);

			// Get solution and register to receive events
			SolutionManager = await GetServiceAsync(typeof(SVsSolution)) as IVsSolution2;
			Assumes.Present(SolutionManager);
			UpdateUnrealLoadedStatus();
			SolutionManager.AdviseSolutionEvents(this, out SolutionEventsHandle);

			// Grab the solution build manager.  We need this in order to change certain things about the Visual
			// Studio environment, like what the active startup project is
			// Get solution build manager
			SolutionBuildManager = await GetServiceAsync(typeof(SVsSolutionBuildManager)) as IVsSolutionBuildManager2;
			Assumes.Present(SolutionBuildManager);
			SolutionBuildManager.AdviseUpdateSolutionEvents(this, out UpdateSolutionEventsHandle);

			// Get debugger
			Debugger = await GetServiceAsync(typeof(SVsShellDebugger)) as IVsDebugger;
			Assumes.Present(Debugger);
			Debugger.AdviseDebuggerEvents(this, out DebuggerEventsHandle);

			// Create our command-line editor
			CommandLineEditor.Initialize();

			// Create our startup project selector
			StartupProjectSelector = new StartupProjectSelector();

			// Create 'BuildStartupProject' instance
			BuildStartupProject = new BuildStartupProject();

			// Create 'CompileSingleFile' instance
			CompileSingleFile = new CompileSingleFile();

			// Create 'GenerateProjectFiles' tools
			GenerateProjectFiles = new GenerateProjectFiles();

			// Create Batch Builder tools
			BatchBuilder.Initialize();

			// Create the project menu quick builder
			QuickBuilder = new QuickBuild();

			BuildAndProfile = new BuildAndProfile();

			// Create 'Perforce menu' instance
			P4CommandsGroup = new P4Commands();

			// Create 'FileBrowser' instance
			FileBrowser = new FileBrowser();

			// Create 'CycleRelatedFiles' instance
			CycleRelatedFiles = new CycleRelatedFiles();

			// Create 'UbaVisualizer' instance
			UbaVisualizer = new UbaVisualizer();

			// Create 'CopyIncludePath' instance
			CopyIncludePath = new CopyIncludePath();

			// Call parent implementation
			base.Initialize();

			if (DTE.Solution.IsOpen)
			{
				StartTicker();
			}

			DTE2.Events.WindowEvents.WindowActivated += WindowEvents_WindowActivated;
		}

		private void WindowEvents_WindowActivated(Window GotFocus, Window LostFocus)
		{
			ThreadHelper.ThrowIfNotOnUIThread();
			if (GotFocus.Document != null)
				OnDocumentActivated(GotFocus.Document);
		}

		private void StartTicker()
		{
			bRefreshTitleInTick = true;
			// Create a "ticker" on a background thread that ticks the package on the UI thread
			Interlocked.Exchange(ref bCancelTicker, 0);
			Ticker = new Thread(TickAsyncMain)
			{
				Priority = ThreadPriority.Lowest
			};
			Ticker.Start();
		}

		private void StopTicker()
		{
			if (bCancelTicker == 0)
			{
				Interlocked.Exchange(ref bCancelTicker, 1);
			}
		}

		/// <summary>
		/// Tick loop on worker thread
		/// </summary>
		private void TickAsyncMain()
		{
			try
			{
				bool bLastTick = false;
				while (true)
				{
					ThreadHelper.JoinableTaskFactory.Run(async () =>
					{
						await ThreadHelper.JoinableTaskFactory.SwitchToMainThreadAsync();
						if (bCancelTicker != 0)
							bLastTick = true;
						Tick(bLastTick);
					});

					if (bLastTick)
						return;
					Thread.Sleep(TickPeriod);
				}
			}
			catch (ThreadAbortException)
			{
			}
		}

		/// <summary>
		/// Tick function on main UI thread
		/// </summary>
		private void Tick(bool bLastTick)
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			TickRefreshTitle(bLastTick);

			BatchBuilder.Tick();
		}

		public void TickRefreshTitle(bool bLastTick)
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			if (bLastTick)
			{
				Utils.SolutionTitle = null;
				return;
			}

			if (bRefreshTitleInTick)
			{
				bRefreshTitleInTick = false;

				if (!UnrealVSPackage.Instance.OptionsPage.IncludeFolderInUE5SolutionName)
				{
					return;
				}

				string SolutionFileName = UnrealVSPackage.Instance.DTE.Solution.FileName;
				string SolutionName = Path.GetFileNameWithoutExtension(SolutionFileName);

				// Only affect UE5 solution
				if (SolutionName != "UE5")
				{
					return;
				}

				string FolderName = Path.GetFileName(Path.GetDirectoryName(SolutionFileName));
				string Mode = "";
				if (DbgMode == DBGMODE.DBGMODE_Run)
					Mode = " (Running)";
				if (DbgMode == DBGMODE.DBGMODE_Break)
					Mode = " (Debugging)";
				string NewTitle = FolderName + '\\' + SolutionName;

				Utils.SolutionTitle = NewTitle;

				Utils.MainWindowTitle = NewTitle + Mode + " - Microsoft Visual Studio";
			}
		}

		/// <summary>
		/// Implementation from IDisposable
		/// </summary>
		public void Dispose()
		{
			ThreadHelper.ThrowIfNotOnUIThread();
			Dispose(true);
		}

		/// IDispose pattern lets us clean up our stuff!
		protected override void Dispose(bool disposing)
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			if (Ticker != null && Ticker.IsAlive)
			{
				Thread.Sleep(TickPeriod + TickPeriod);
				if (Ticker.IsAlive)
				{
					Logging.WriteLine("WARNING: Force aborting Ticker thread");
					Ticker.Abort();
				}
			}

			base.Dispose(disposing);

			// Clean up singleton instance
			PrivateInstance = null;

			StartupProjectSelector = null;
			BatchBuilder = null;
			QuickBuilder = null;
			BuildAndProfile?.Dispose();
			BuildAndProfile = null;

			CompileSingleFile?.Dispose();
			CompileSingleFile = null;

			P4CommandsGroup?.Dispose();
			P4CommandsGroup = null;

			CopyIncludePath?.Dispose();
			CopyIncludePath = null;

			CommandLineEditor?.Dispose();
			CommandLineEditor = null;

			// No longer want solution events
			if (SolutionEventsHandle != 0)
			{
				SolutionManager.UnadviseSolutionEvents(SolutionEventsHandle);
				SolutionEventsHandle = 0;
			}
			SolutionManager = null;

			// No longer want selection events
			if (SelectionEventsHandle != 0)
			{
				SelectionManager.UnadviseSelectionEvents(SelectionEventsHandle);
				SelectionEventsHandle = 0;
			}
			SelectionManager = null;

			if (DebuggerEventsHandle != 0)
				Debugger.UnadviseDebuggerEvents(DebuggerEventsHandle);
			Debugger = null;


			// No longer want update solution events
			if (UpdateSolutionEventsHandle != 0)
			{
				SolutionBuildManager.UnadviseUpdateSolutionEvents(UpdateSolutionEventsHandle);
				UpdateSolutionEventsHandle = 0;
			}
			SolutionBuildManager = null;

			Logging.WriteLine("Closing UnrealVS extension");
			Logging.Close();
		}


		/// Visual Studio shell DTE interface
		public DTE DTE
		{
			get;
			private set;
		}

		/// Visual Studio shell DTE2 interface
		private DTE2 _DTE2;
		public DTE2 DTE2
		{
			get
			{
				ThreadHelper.ThrowIfNotOnUIThread();
				if (_DTE2 == null)
				{
					// Get the interface when first used.
					// This method fails during the Initialize() of the 
					// package due to the strange method it requires.
					_DTE2 = GetDTE2ForCurrentInstance(DTE);
				}
				return _DTE2;
			}
		}

		//public IVsTextManager3 TextManager
		//{
		//	get;
		//	private set;
		//}

		public IEditorOptionsFactoryService EditorOptionsFactory
		{
			get;
			private set;
		}

		/// The package's options page
		public UnrealVsOptions OptionsPage
		{
			get { return (UnrealVsOptions)GetDialogPage(typeof(UnrealVsOptions)); }
		}

		/// <summary>
		/// Launches a program
		/// </summary>
		/// <param name="ProgramFile">Path to the program to run</param>
		/// <param name="Arguments">Command-line arguments</param>
		/// <param name="OnExit">Optional callback whent he program exits</param>
		/// <param name="OutputHandler">If supplied, std-out and std-error will be redirected to this function and no shell window will be created</param>
		/// <returns>The newly-created process, or null if it wasn't able to start.  Exceptions are swallowed, but a debug output string is emitted on errors</returns>
		public static System.Diagnostics.Process LaunchProgram(string ProgramFile, string Arguments, EventHandler OnExit = null, DataReceivedEventHandler OutputHandler = null, bool bWaitForCompletion = false)
		{
			// Create the action's process.
			ProcessStartInfo ActionStartInfo = new ProcessStartInfo
			{
				FileName = ProgramFile,
				Arguments = Arguments
			};

			if (OutputHandler != null)
			{
				ActionStartInfo.RedirectStandardInput = true;
				ActionStartInfo.RedirectStandardOutput = true;
				ActionStartInfo.RedirectStandardError = true;

				// True to use a DOS box to run the program in, otherwise false to run directly.  In order to redirect
				// output, UseShellExecute must be disabled
				ActionStartInfo.UseShellExecute = false;

				// Don't show the DOS box, since we're redirecting everything
				ActionStartInfo.CreateNoWindow = true;
			}


			Logging.WriteLine(String.Format("Executing: {0} {1}", ActionStartInfo.FileName, ActionStartInfo.Arguments));

			System.Diagnostics.Process ActionProcess;

			try
			{
				ActionProcess = new System.Diagnostics.Process
				{
					StartInfo = ActionStartInfo
				};

				if (OnExit != null)
				{
					ActionProcess.EnableRaisingEvents = true;
					ActionProcess.Exited += OnExit;
				}

				if (ActionStartInfo.RedirectStandardOutput)
				{
					ActionProcess.EnableRaisingEvents = true;
					ActionProcess.OutputDataReceived += OutputHandler;
					ActionProcess.ErrorDataReceived += OutputHandler;
				}

				// Launch the program
				ActionProcess.Start();

				if (ActionStartInfo.RedirectStandardOutput)
				{
					ActionProcess.BeginOutputReadLine();
					ActionProcess.BeginErrorReadLine();
				}

				if (bWaitForCompletion)
				{
					while ((ActionProcess != null) && (!ActionProcess.HasExited))
					{
						ActionProcess.WaitForExit(50);
					}
				}
			}
			catch (Exception Ex)
			{
				// Couldn't launch program
				Logging.WriteLine("Couldn't launch program: " + ActionStartInfo.FileName);
				Logging.WriteLine("Exception: " + Ex.Message);
				ActionProcess = null;
			}

			return ActionProcess;
		}


		/// <summary>
		/// Gets a Visual Studio pane to output text to, or creates one if not visible.  Does not bring the pane to front (you can call Activate() to do that.)
		/// </summary>
		/// <returns>The pane to output to, or null on error</returns>
		public IVsOutputWindowPane GetOutputPane()
		{
			return GetOutputPane(VSConstants.OutputWindowPaneGuid.BuildOutputPane_guid, "Build");
		}

		/// <summary>
		/// Gets a Visual Studio pane to output text to for P4 ops, or creates one if not visible.  Does not bring the pane to front (you can call Activate() to do that.)
		/// </summary>
		/// <returns>The pane to output to, or null on error</returns>
		public IVsOutputWindowPane GetP4OutputPane()
		{
			return GetOutputPane(VSConstants.OutputWindowPaneGuid.GeneralPane_guid, "UnrealVS.P4");
		}

		public IEnumerable<string> GetLoadedProjectPaths()
		{
			return LoadedProjectPaths;
		}

		/// <summary>
		/// Overrides Package.OnLoadOptions()
		/// Invoked by the package class when there are options to be read out of the solution file.
		/// </summary>
		/// <param name="key">The name of the option key to load.</param>
		/// <param name="stream">The stream to load the option data from.</param>
		protected override void OnLoadOptions(string key, Stream stream)
		{
			Logging.WriteLine("Loading Options for key: " + key);
			try
			{
				if (0 == string.Compare(key, CommandLineOptionKey))
				{
					Logging.WriteLine("Restoring CommandLineEditor options");
					CommandLineEditor.LoadOptions(stream);
				}
				else if (0 == string.Compare(key, BatchBuildSetsOptionKey))
				{
					Logging.WriteLine("Restoring BatchBuilder options");
					BatchBuilder.LoadOptions(stream);
				}
			}
			catch (Exception Ex)
			{
				// Couldn't load options
				Exception AppEx = new ApplicationException("OnLoadOptions() failed with key " + key, Ex);
				Logging.WriteLine(AppEx.ToString());
				throw AppEx;
			}
		}


		/// <summary>
		/// Overrides Package.OnSaveOptions()
		/// Invoked by the Package class when there are options to be saved to the solution file.
		/// </summary>
		/// <param name="key">The name of the option key to save.</param>
		/// <param name="stream">The stream to save the option data to.</param>
		protected override void OnSaveOptions(string key, Stream stream)
		{
			try
			{
				if (0 == string.Compare(key, CommandLineOptionKey))
				{
					Logging.WriteLine("Saving CommandLineEditor options");
					CommandLineEditor.SaveOptions(stream);
				}
				else if (0 == string.Compare(key, BatchBuildSetsOptionKey))
				{
					Logging.WriteLine("Saving BatchBuilder options");
					BatchBuilder.SaveOptions(stream);
				}
			}
			catch (Exception Ex)
			{
				// Couldn't save options
				Exception AppEx = new ApplicationException("OnSaveOptions() failed with key " + key, Ex);
				Logging.WriteLine(AppEx.ToString());
				throw AppEx;
			}
		}


		///
		/// IVsSolutionEvents implementation
		///

		int IVsSolutionEvents.OnAfterCloseSolution(object pUnkReserved)
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			UpdateUnrealLoadedStatus();

			OnSolutionClosed?.Invoke();

			return VSConstants.S_OK;
		}

		int IVsSolutionEvents.OnAfterLoadProject(IVsHierarchy pStubHierarchy, IVsHierarchy pRealHierarchy)
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			var LoadedProject = Utils.HierarchyObjectToProject(pRealHierarchy);
			if (LoadedProject != null && OnProjectLoaded != null)
				OnProjectLoaded.Invoke(LoadedProject);
			return VSConstants.S_OK;
		}

		int IVsSolutionEvents.OnAfterOpenProject(IVsHierarchy pHierarchy, int fAdded)
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			// This function is called after a Visual Studio project is opened (or a new project is created.)

			// Get the actual Project object from the IVsHierarchy object that was supplied
			var OpenedProject = Utils.HierarchyObjectToProject(pHierarchy);
			Utils.OnProjectListChanged();
			if (OpenedProject != null && OnProjectOpened != null)
			{
				LoadedProjectPaths.Add(OpenedProject.FullName);
				OnProjectOpened(OpenedProject);
			}

			return VSConstants.S_OK;
		}

		int IVsSolutionEvents.OnAfterOpenSolution(object pUnkReserved, int fNewSolution)
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			UpdateUnrealLoadedStatus();

			StartTicker();

			OnSolutionOpened?.Invoke();

			return VSConstants.S_OK;
		}

		int IVsSolutionEvents.OnBeforeCloseProject(IVsHierarchy pHierarchy, int fRemoved)
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			// This function is called after a Visual Studio project is closed

			// Get the actual Project object from the IVsHierarchy object that was supplied
			var ClosedProject = Utils.HierarchyObjectToProject(pHierarchy);
			if (ClosedProject != null && OnProjectClosed != null)
			{
				LoadedProjectPaths.Remove(ClosedProject.FullName);
				OnProjectClosed(ClosedProject);
			}

			return VSConstants.S_OK;
		}

		int IVsSolutionEvents.OnBeforeCloseSolution(object pUnkReserved)
		{
			StopTicker();

			OnSolutionClosing?.Invoke();

			return VSConstants.S_OK;
		}

		int IVsSolutionEvents.OnBeforeUnloadProject(IVsHierarchy pRealHierarchy, IVsHierarchy pStubHierarchy)
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			var UnloadedProject = Utils.HierarchyObjectToProject(pRealHierarchy);
			if (UnloadedProject != null && OnProjectUnloading != null)
				OnProjectUnloading.Invoke(UnloadedProject);
			return VSConstants.S_OK;
		}

		int IVsSolutionEvents.OnQueryCloseProject(IVsHierarchy pHierarchy, int fRemoving, ref int pfCancel)
		{
			return VSConstants.S_OK;
		}

		int IVsSolutionEvents.OnQueryCloseSolution(object pUnkReserved, ref int pfCancel)
		{
			if (BatchBuilder.IsBusy)
			{
				pfCancel = 1;
			}

			return VSConstants.S_OK;
		}

		int IVsSolutionEvents.OnQueryUnloadProject(IVsHierarchy pRealHierarchy, ref int pfCancel)
		{
			return VSConstants.S_OK;
		}


		///
		/// IVsSelectionEvents implementation
		///

		int IVsSelectionEvents.OnCmdUIContextChanged(uint dwCmdUICookie, int fActive)
		{
			OnUIContextChanged?.Invoke(dwCmdUICookie, fActive != 0);

			return VSConstants.S_OK;
		}

		int IVsSelectionEvents.OnElementValueChanged(uint elementid, object varValueOld, object varValueNew)
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			// This function is called when selection changes in various Visual Studio tool windows
			// and sub-systems.

			// Handle startup project changes
			if (elementid == (uint)VSConstants.VSSELELEMID.SEID_StartupProject)
			{
				// If we are registered to a project hierarchy for events, unregister
				var OldStartupProjectHierarchy = (IVsHierarchy)varValueOld;
				if (OldStartupProjectHierarchy != null && ProjectHierarchyEventsHandle != 0)
				{
					OldStartupProjectHierarchy.UnadviseHierarchyEvents(ProjectHierarchyEventsHandle);
					ProjectHierarchyEventsHandle = 0;
				}

				Project NewStartupProject = null;

				// Incoming hierarchy object could be null (if no startup project is set yet, or during shutdown.)
				var NewStartupProjectHierarchy = (IVsHierarchy)varValueNew;
				if (NewStartupProjectHierarchy != null)
				{
					// Get the actual Project object from the IVsHierarchy object that was supplied
					NewStartupProject = Utils.HierarchyObjectToProject(NewStartupProjectHierarchy);

					if (NewStartupProject != null)
					{
						// Register for events from the project
						NewStartupProjectHierarchy.AdviseHierarchyEvents(this, out ProjectHierarchyEventsHandle);
					}
				}

				if (NewStartupProject != null)
				{
					OnStartupProjectChanged(NewStartupProject);
				}
			}

			return VSConstants.S_OK;
		}

		int IVsSelectionEvents.OnSelectionChanged(IVsHierarchy pHierOld, uint itemidOld, IVsMultiItemSelect pMISOld,
												  ISelectionContainer pSCOld, IVsHierarchy pHierNew, uint itemidNew,
												  IVsMultiItemSelect pMISNew, ISelectionContainer pSCNew)
		{
			return VSConstants.S_OK;
		}


		// IVsHierarchyEvents Interface

		Int32 IVsHierarchyEvents.OnItemAdded(UInt32 itemidParent, UInt32 itemidSiblingPrev, UInt32 itemidAdded)
		{
			return VSConstants.S_OK;
		}

		Int32 IVsHierarchyEvents.OnPropertyChanged(UInt32 itemid, Int32 propid, UInt32 flags)
		{
			OnStartupProjectPropertyChanged?.Invoke(itemid, propid, flags);
			return VSConstants.S_OK;
		}

		Int32 IVsHierarchyEvents.OnItemsAppended(UInt32 itemidParent)
		{
			return VSConstants.S_OK;
		}

		Int32 IVsHierarchyEvents.OnItemDeleted(UInt32 itemid)
		{
			return VSConstants.S_OK;
		}

		Int32 IVsHierarchyEvents.OnInvalidateItems(UInt32 itemidParent)
		{
			return VSConstants.S_OK;
		}

		Int32 IVsHierarchyEvents.OnInvalidateIcon(IntPtr hicon)
		{
			return VSConstants.S_OK;
		}


		#region IVsPersistSolutionProps
		int IVsPersistSolutionProps.SaveUserOptions(IVsSolutionPersistence pPersistence)
		{
			return VSConstants.S_OK;
		}

		int IVsPersistSolutionProps.LoadUserOptions(IVsSolutionPersistence pPersistence, uint grfLoadOpts)
		{
			return VSConstants.S_OK;
		}

		int IVsPersistSolutionProps.WriteUserOptions(IStream pOptionsStream, string pszKey)
		{
			return VSConstants.S_OK;
		}

		int IVsPersistSolutionProps.ReadUserOptions(IStream pOptionsStream, string pszKey)
		{
			return VSConstants.S_OK;
		}

		int IVsPersistSolutionProps.QuerySaveSolutionProps(IVsHierarchy pHierarchy, VSQUERYSAVESLNPROPS[] pqsspSave)
		{
			return VSConstants.S_OK;
		}

		int IVsPersistSolutionProps.SaveSolutionProps(IVsHierarchy pHierarchy, IVsSolutionPersistence pPersistence)
		{
			return VSConstants.S_OK;
		}

		int IVsPersistSolutionProps.WriteSolutionProps(IVsHierarchy pHierarchy, string pszKey, IPropertyBag pPropBag)
		{
			return VSConstants.S_OK;
		}

		int IVsPersistSolutionProps.ReadSolutionProps(IVsHierarchy pHierarchy, string pszProjectName, string pszProjectMk, string pszKey, int fPreLoad, IPropertyBag pPropBag)
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			if (!string.Equals(pszKey, GuidList.UnrealVSPackageString, StringComparison.InvariantCultureIgnoreCase))
				return VSConstants.S_OK;

			pPropBag.Read("AvailablePlatforms", out object availablePlatformsObject, null, (uint)VarEnum.VT_BSTR, pPropBag);

			string[] availablePlatforms = null;
			if (availablePlatformsObject != null)
			{
				string availablePlatformsString = availablePlatformsObject as string;
				availablePlatforms = availablePlatformsString.Split(';');
			}

			UnrealSolutionProperties = new UnrealSolutionProperties(availablePlatforms);

			// a UnrealVS section was found so we consider this solution a unreal solution
			IsUESolutionLoaded = true;

			return VSConstants.S_OK;
		}

		int IVsPersistSolutionProps.OnProjectLoadFailure(IVsHierarchy pStubHierarchy, string pszProjectName, string pszProjectMk, string pszKey)
		{
			return VSConstants.S_OK;
		}

		#endregion

		int IVsDebuggerEvents.OnModeChange(DBGMODE dbgmodeNew)
		{
			DbgMode = dbgmodeNew;
			bRefreshTitleInTick = true;
			return VSConstants.S_OK;
		}


		// IVsUpdateSolutionEvents Interface

		public int UpdateSolution_Begin(ref int pfCancelUpdate)
		{
			OnBuildBegin?.Invoke(out pfCancelUpdate);
			return VSConstants.S_OK;
		}

		public int UpdateSolution_Done(int fSucceeded, int fModified, int fCancelCommand)
		{
			OnBuildDone?.Invoke(fSucceeded != 0, fModified != 0, fCancelCommand != 0);
			return VSConstants.S_OK;
		}

		public int UpdateSolution_StartUpdate(ref int pfCancelUpdate)
		{
			return VSConstants.S_OK;
		}

		public int UpdateSolution_Cancel()
		{
			return VSConstants.S_OK;
		}

		public int OnActiveProjectCfgChange(IVsHierarchy pIVsHierarchy)
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			// This function is called after a Visual Studio project has its active config changed

			// Check whether the project is the current startup project
			SolutionBuildManager.get_StartupProject(out IVsHierarchy StartupProjectHierarchy);
			if (StartupProjectHierarchy != null && StartupProjectHierarchy == pIVsHierarchy)
			{
				// Get the actual Project object from the IVsHierarchy object that was supplied
				var Project = Utils.HierarchyObjectToProject(pIVsHierarchy);
				if (Project != null && OnStartupProjectConfigChanged != null)
				{
					OnStartupProjectConfigChanged(Project);
				}
			}
			return VSConstants.S_OK;
		}

		private void UpdateUnrealLoadedStatus()
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			if (!DTE.Solution.IsOpen)
			{
				IsUESolutionLoaded = false;
				return;
			}

			SolutionManager.GetSolutionInfo(out string SolutionDirectory, out _SolutionFilepath, out string UserOptsFile);

			// if ReadProps found a valud UE solution we do not need to check these legacy definitions of a solution
			if (_IsUESolutionLoaded.HasValue)
				return;

			// Legacy paths for determing if a solution is a unreal solution by checking for tags & solution name.
			string[] SolutionLines = Array.Empty<string>();
			try
			{
				SolutionLines = File.ReadAllLines(_SolutionFilepath);
			}
			catch
			{
			}

			const string UBTTag = "# UnrealEngineGeneratedSolutionVersion=";
			var UBTLine = SolutionLines.FirstOrDefault(TextLine => TextLine.Trim().StartsWith(UBTTag));
			if (UBTLine != null)
			{
				_UBTVersion = UBTLine.Trim().Substring(UBTTag.Length);
				IsUESolutionLoaded = true;
			}
			else
			{
				_UBTVersion = string.Empty;
				IsUESolutionLoaded =
					(
						_SolutionFilepath != null &&
						Path.GetFileName(_SolutionFilepath).StartsWith(UnrealSolutionFileNamePrefix, StringComparison.OrdinalIgnoreCase)
					);
			}
		}

		/** Private Fields & Properties */

		private static UnrealVSPackage PrivateInstance = null;

		/// Handle that we used at shutdown to unregister for selection manager events
		private UInt32 SelectionEventsHandle;

		/// Handle that we use at shutdown to unregister for events about solution activity
		private UInt32 SolutionEventsHandle;

		/// Handle that we use at shutdown to unregister for events about solution build activity
		private UInt32 UpdateSolutionEventsHandle;

		/// Handle that we use at shutdown to unregister for events about solution build activity
		private UInt32 DebuggerEventsHandle;

		/// Handle that we use to unregister for events about startup project hierarchy activity
		UInt32 ProjectHierarchyEventsHandle;

		/// Our command-line editing component
		private CommandLineEditor CommandLineEditor = new CommandLineEditor();

		/// BuildStartupProject feature
		private BuildStartupProject BuildStartupProject;

		/// FileBrowser feature
		private FileBrowser FileBrowser;

		/// CycleRelatedFiles feature
		private CycleRelatedFiles CycleRelatedFiles;

		/// UbaVisualizer feature
		private UbaVisualizer UbaVisualizer;

		/// CompileSingleFile feature
		private CompileSingleFile CompileSingleFile;

		/// Our Build and Profile component
		private BuildAndProfile BuildAndProfile;

		/// Project file generator button
		private GenerateProjectFiles GenerateProjectFiles;

		/// Batch Builder button/command handler
		private BatchBuilder BatchBuilder = new BatchBuilder();

		/// CopyIncludePath feature
		private CopyIncludePath CopyIncludePath;

		/// Perforce features
		private P4Commands P4CommandsGroup;

		/// Ticker thread
		private Thread Ticker;

		/// Ticker thread cancel flag
		private int bCancelTicker = 0;

		private string _UBTVersion = string.Empty;

		private readonly List<string> LoadedProjectPaths = new List<string>();
		private bool? _IsUESolutionLoaded;

		private DBGMODE DbgMode = DBGMODE.DBGMODE_Design;
		private bool bRefreshTitleInTick;

		/// Obtains the DTE2 interface for this instance of VS from the RunningObjectTable
		private static DTE2 GetDTE2ForCurrentInstance(DTE DTE)
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			// Find the ROT entry for visual studio running under current process.
			int HResult = NativeMethods.GetRunningObjectTable(0, out IRunningObjectTable Rot);
			if (HResult == 0)
			{
				Rot.EnumRunning(out IEnumMoniker EnumMoniker);
				EnumMoniker.Reset();
				IMoniker[] Moniker = new IMoniker[1];
				while (EnumMoniker.Next(1, Moniker, out _) == 0)
				{
					Rot.GetObject(Moniker[0], out object ComObject);

					if (ComObject is DTE2 CandidateDTE2)
					{
						if (CandidateDTE2.DTE == DTE)
						{
							return CandidateDTE2;
						}
					}
				}
			}
			return null;
		}

	}

	internal static class NativeMethods
	{
		/// ROT function in ole32.dll needed by GetDTE2ForCurrentInstance()
		[DllImport("ole32.dll")]
		internal static extern int GetRunningObjectTable(int reserved, out IRunningObjectTable prot);

		public enum MapType : uint
		{
			MAPVK_VK_TO_VSC = 0x0,
			MAPVK_VSC_TO_VK = 0x1,
			MAPVK_VK_TO_CHAR = 0x2,
			MAPVK_VSC_TO_VK_EX = 0x3,
		}

		[DllImport("user32.dll")]
		public static extern int ToUnicode(
			uint wVirtKey,
			uint wScanCode,
			byte[] lpKeyState,
			[Out, MarshalAs(UnmanagedType.LPWStr, SizeParamIndex = 4)]
			StringBuilder pwszBuff,
			int cchBuff,
			uint wFlags);

		[DllImport("user32.dll")]
		public static extern bool GetKeyboardState(byte[] lpKeyState);

		[DllImport("user32.dll")]
		public static extern uint MapVirtualKey(uint uCode, MapType uMapType);

		[DllImport("user32.dll", EntryPoint = "CreateWindowEx", CharSet = CharSet.Unicode)]
		internal static extern IntPtr CreateWindowEx(int dwExStyle,
													  string lpszClassName,
													  string lpszWindowName,
													  int style,
													  int x, int y,
													  int width, int height,
													  IntPtr hwndParent,
													  IntPtr hMenu,
													  IntPtr hInst,
													  [MarshalAs(UnmanagedType.AsAny)] object pvParam);

		[DllImport("user32.dll", EntryPoint = "DestroyWindow", CharSet = CharSet.Unicode)]
		internal static extern bool DestroyWindow(IntPtr hwnd);

		[DllImport("user32.dll", EntryPoint = "SetWindowTextW", CharSet = CharSet.Unicode)]
		internal static extern bool SetWindowTextW(IntPtr hWnd, string lpString);

		[DllImport("user32.dll", EntryPoint = "SetWindowPos", CharSet = CharSet.Unicode)]
		internal static extern bool SetWindowPos(IntPtr hWnd, IntPtr hWndInsertAfter, int X, int Y, int cx, int cy, uint uFlags);

		[DllImport("user32.dll", EntryPoint = "UpdateWindow", CharSet = CharSet.Unicode)]
		internal static extern bool UpdateWindow(IntPtr hWnd);

		[DllImport("user32.dll", EntryPoint = "IsWindow", CharSet = CharSet.Unicode)]
		internal static extern bool IsWindow(IntPtr hWnd);

		[DllImport("user32.dll", EntryPoint = "PostMessageW", CharSet = CharSet.Unicode)]
		internal static extern bool PostMessageW(IntPtr hWnd, int Msg, IntPtr wParam, IntPtr lParam);

		[DllImport("user32.dll", CharSet = CharSet.Unicode)]
		internal static extern IntPtr SetFocus(IntPtr hWnd);

		[DllImport("user32.dll")]
		internal static extern IntPtr LoadCursorW(IntPtr hInstance, IntPtr lpCursorName);
		

				[DllImport("user32.dll", EntryPoint = "RegisterClassExW", SetLastError = true)]
		internal static extern UInt16 RegisterClassExW(ref WNDCLASSEX lpWndClass);

		[DllImport("user32.dll")]
		internal static extern IntPtr DefWindowProc(IntPtr hWnd, uint uMsg, IntPtr wParam, IntPtr lParam);

		[StructLayout(LayoutKind.Sequential, CharSet = CharSet.Unicode)]
		internal struct WNDCLASSEX
		{
			[MarshalAs(UnmanagedType.U4)]
			public int cbSize;
			[MarshalAs(UnmanagedType.U4)]
			public int style;
			public IntPtr lpfnWndProc; // not WndProc
			public int cbClsExtra;
			public int cbWndExtra;
			public IntPtr hInstance;
			public IntPtr hIcon;
			public IntPtr hCursor;
			public IntPtr hbrBackground;
			public string lpszMenuName;
			public string lpszClassName;
			public IntPtr hIconSm;
		}
	}
}
