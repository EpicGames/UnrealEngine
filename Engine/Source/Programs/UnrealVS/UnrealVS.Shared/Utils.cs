// Copyright Epic Games, Inc. All Rights Reserved.

using EnvDTE;
using EnvDTE80;
using Microsoft.VisualStudio;
using Microsoft.VisualStudio.Shell;
using Microsoft.VisualStudio.Shell.Interop;
using System;
using System.Collections.Generic;
using System.ComponentModel;
using System.IO;
using System.Linq;
using System.Runtime.InteropServices;
using System.Windows.Controls;
using System.Windows.Media;
using System.Xml;

namespace UnrealVS
{
	public class Utils
	{
		public const string UProjectExtension = "uproject";

		public class SafeProjectReference
		{
			public string FullName { get; set; }
			public string Name { get; set; }

			public Project GetProjectSlow()
			{
				ThreadHelper.ThrowIfNotOnUIThread();

				Project[] Projects = GetAllProjectsFromDTE();
#pragma warning disable VSTHRD010 // Invoke single-threaded types on Main thread
				return Projects.FirstOrDefault(Proj => string.CompareOrdinal(Proj.FullName, FullName) == 0);
#pragma warning restore VSTHRD010 // Invoke single-threaded types on Main thread
			}
		}

		/// <summary>
		/// Converts a Project to an IVsHierarchy
		/// </summary>
		/// <param name="Project">Project object</param>
		/// <returns>IVsHierarchy for the specified project</returns>
		public static IVsHierarchy ProjectToHierarchyObject(Project Project)
		{
			ThreadHelper.ThrowIfNotOnUIThread();
			UnrealVSPackage.Instance.SolutionManager.GetProjectOfUniqueName(Project.FullName, out IVsHierarchy HierarchyObject);
			return HierarchyObject;
		}


		/// <summary>
		/// Converts an IVsHierarchy object to a Project
		/// </summary>
		/// <param name="HierarchyObject">IVsHierarchy object</param>
		/// <returns>Visual Studio project object</returns>
		public static Project HierarchyObjectToProject(IVsHierarchy HierarchyObject)
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			// Get the actual Project object from the IVsHierarchy object that was supplied
			HierarchyObject.GetProperty(VSConstants.VSITEMID_ROOT, (int)__VSHPROPID.VSHPROPID_ExtObject, out object ProjectObject);
			return (Project)ProjectObject;
		}

		/// <summary>
		/// Get the project selected in the solution explorer by accessing the current selection and converting to a Project if possible.
		/// </summary>
		/// <returns></returns>
		public static Project GetCurrentlySelectedProject()
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			IntPtr HierarchyPtr = IntPtr.Zero;
			IntPtr SelectionContainerPtr = IntPtr.Zero;

			try
			{
				UnrealVSPackage.Instance.SelectionManager.GetCurrentSelection(out HierarchyPtr, out uint ProjectItemId, out _, out SelectionContainerPtr);
				if (HierarchyPtr == IntPtr.Zero)
				{
					return null;
				}

				object SelectedHierarchyObj = Marshal.GetObjectForIUnknown(HierarchyPtr);
				if (!(SelectedHierarchyObj is IVsHierarchy SelectedHierarchy))
				{
					return null;
				}

				return HierarchyObjectToProject(SelectedHierarchy);
			}
			finally
			{
				if (HierarchyPtr != IntPtr.Zero)
				{
					Marshal.Release(HierarchyPtr);
				}
				if (SelectionContainerPtr != IntPtr.Zero)
				{
					Marshal.Release(SelectionContainerPtr);
				}
			}
		}

		private static readonly string BuildModuleSuffix = ".Build.cs";

		public static bool GetSelectedModule(out Project ModuleProject, out string ModuleName)
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			ModuleProject = null;
			ModuleName = null;

			IntPtr HierarchyPtr = IntPtr.Zero;
			IntPtr SelectionContainerPtr = IntPtr.Zero;

			try
			{
				UnrealVSPackage.Instance.SelectionManager.GetCurrentSelection(out HierarchyPtr, out uint ProjectItemId, out _, out SelectionContainerPtr);
				if (HierarchyPtr == IntPtr.Zero)
				{
					return false;
				}

				object SelectedHierarchyObj = Marshal.GetObjectForIUnknown(HierarchyPtr);
				if (!(SelectedHierarchyObj is IVsHierarchy SelectedHierarchy))
				{
					return false;
				}

				// Walk over the children and look for a .Build.cs file. If one exists, then this is a Module folder.
				SelectedHierarchy.GetProperty(ProjectItemId, (int)__VSHPROPID.VSHPROPID_FirstChild, out object FirstChildIdObj);

				uint GetItemID(object IdObject)
				{
					if (IdObject is null)
					{
						return VSConstants.VSITEMID_NIL;
					}
					if (IdObject is int IntId)
					{
						return (uint)IntId;
					}
					if (IdObject is uint UintId)
					{
						return UintId;
					}
					return VSConstants.VSITEMID_NIL;
				}

				if (FirstChildIdObj != null)
				{
					uint ChildObjectID = GetItemID(FirstChildIdObj);

					while (ChildObjectID != VSConstants.VSITEMID_NIL)
					{
						SelectedHierarchy.GetProperty(ChildObjectID, (int)__VSHPROPID.VSHPROPID_Name, out object ObjectNameObj);

						if (ObjectNameObj is string ObjectName && ObjectName.EndsWith(BuildModuleSuffix, StringComparison.InvariantCultureIgnoreCase))
						{
							SelectedHierarchy.GetProperty(ProjectItemId, (int)__VSHPROPID.VSHPROPID_ExtObject, out object ProjectObject);

							if (ProjectObject is ProjectItem SelectedItem)
							{
								ModuleName = ObjectName.Substring(0, ObjectName.Length - BuildModuleSuffix.Length);
								ModuleProject = SelectedItem.ContainingProject;
								return true;
							}
						}

						SelectedHierarchy.GetProperty(ChildObjectID, (int)__VSHPROPID.VSHPROPID_NextSibling, out object SiblingIdObj);

						ChildObjectID = GetItemID(SiblingIdObj);
					}
				}

				return false;
			}
			finally
			{
				if (HierarchyPtr != IntPtr.Zero)
				{
					Marshal.Release(HierarchyPtr);
				}
				if (SelectionContainerPtr != IntPtr.Zero)
				{
					Marshal.Release(SelectionContainerPtr);
				}
			}
		}

		/// <summary>
		/// Converts an IVsHierarchy object to a config provider interface
		/// </summary>
		/// <param name="HierarchyObject">IVsHierarchy object</param>
		/// <returns>Visual Studio project object</returns>
		public static IVsCfgProvider2 HierarchyObjectToCfgProvider(IVsHierarchy HierarchyObject)
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			// Get the actual Project object from the IVsHierarchy object that was supplied
			HierarchyObject.GetProperty(VSConstants.VSITEMID_ROOT, (int)__VSHPROPID.VSHPROPID_BrowseObject, out object BrowseObject);

			IVsCfgProvider2 CfgProvider = null;
			if (BrowseObject != null)
			{
				CfgProvider = GetCfgProviderFromObject(BrowseObject);
			}

			if (CfgProvider == null)
			{
				CfgProvider = GetCfgProviderFromObject(HierarchyObject);
			}

			return CfgProvider;
		}

		private static IVsCfgProvider2 GetCfgProviderFromObject(object SomeObject)
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			IVsCfgProvider2 CfgProvider2 = null;

			if (SomeObject is IVsGetCfgProvider GetCfgProvider)
			{
				GetCfgProvider.GetCfgProvider(out IVsCfgProvider CfgProvider);
				if (CfgProvider != null)
				{
					CfgProvider2 = CfgProvider as IVsCfgProvider2;
				}
			}

			if (CfgProvider2 == null)
			{
				CfgProvider2 = SomeObject as IVsCfgProvider2;
			}

			return CfgProvider2;
		}

		/// <summary>
		/// Locates a specific project property for the active configuration and returns it (or null if not found.)
		/// </summary>
		/// <param name="Project">Project to search for the property</param>
		/// <param name="PropertyName">Name of the property</param>
		/// <returns>Property object or null if not found</returns>
		public static Property GetProjectProperty(Project Project, string PropertyName)
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			var Properties = Project.Properties;
			if (Properties != null)
			{
				foreach (var RawProperty in Properties)
				{
					var Property = (Property)RawProperty;
					if (Property.Name.Equals(PropertyName, StringComparison.InvariantCultureIgnoreCase))
					{
						return Property;
					}
				}
			}

			// Not found
			return null;
		}

		/// <summary>
		/// Locates a specific project property for the active configuration and attempts to set its value
		/// </summary>
		/// <param name="Property">The property object to set</param>
		/// <param name="PropertyValue">Value to set for this property</param>
		public static void SetPropertyValue(Property Property, object PropertyValue)
		{
			ThreadHelper.ThrowIfNotOnUIThread();
			Property.Value = PropertyValue;

			// @todo: Not sure if actually needed for command-line property (saved in .user files, not in project)
			// Mark the project as modified
			// @todo: Throws exception for C++ projects, doesn't mark as saved
			//				Project.IsDirty = true;
			//				Project.Saved = false;
		}

		/// <summary>
		/// Helper class used by the GetUIxxx functions below.
		/// Callers use this to easily traverse UIHierarchies.
		/// </summary>
		public class UITreeItem
		{
			public UIHierarchyItem Item { get; set; }
			public UITreeItem[] Children { get; set; }
			public string Name
			{ 
				get 
				{
					ThreadHelper.ThrowIfNotOnUIThread();
					return Item != null ? Item.Name : "None"; 
				} 
			}
			public object Object { get { return Item?.Object; } }
		}

		/// <summary>
		/// Converts a UIHierarchy into an easy to use tree of helper class UITreeItem.
		/// </summary>
		public static UITreeItem GetUIHierarchyTree(UIHierarchy Hierarchy)
		{
			ThreadHelper.ThrowIfNotOnUIThread();
			return new UITreeItem
			{
				Item = null,
				Children = (from UIHierarchyItem Child in Hierarchy.UIHierarchyItems select GetUIHierarchyTree(Child)).ToArray()
			};
		}

		/// <summary>
		/// Called by the public GetUIHierarchyTree() function above.
		/// </summary>
		private static UITreeItem GetUIHierarchyTree(UIHierarchyItem HierarchyItem)
		{
			ThreadHelper.ThrowIfNotOnUIThread();
			return new UITreeItem
			{
				Item = HierarchyItem,
				Children = (from UIHierarchyItem Child in HierarchyItem.UIHierarchyItems select GetUIHierarchyTree(Child)).ToArray()
			};
		}

		/// <summary>
		/// Helper function to easily extract a list of objects of type T from a UIHierarchy tree.
		/// </summary>
		/// <typeparam name="T">The type of object to find in the tree. Extracts everything that "Is a" T.</typeparam>
		/// <param name="RootItem">The root of the UIHierarchy to search (converted to UITreeItem via GetUIHierarchyTree())</param>
		/// <returns>An enumerable of objects of type T found beneath the root item.</returns>
		public static IEnumerable<T> GetUITreeItemObjectsByType<T>(UITreeItem RootItem) where T : class
		{
			List<T> Results = new List<T>();

			if (RootItem.Object is T Obj)
			{
				Results.Add(Obj);
			}
			foreach (var Child in RootItem.Children)
			{
				Results.AddRange(GetUITreeItemObjectsByType<T>(Child));
			}

			return Results;
		}

		public static IEnumerable<UIHierarchyItem> GetUITreeItemsByObjectType<T>(UITreeItem RootItem) where T : class
		{
			List<UIHierarchyItem> Results = new List<UIHierarchyItem>();

			if (RootItem.Object is T)
			{
				Results.Add(RootItem.Item);
			}
			foreach (var Child in RootItem.Children)
			{
				Results.AddRange(GetUITreeItemsByObjectType<T>(Child));
			}

			return Results;
		}

		/// <summary>
		/// Helper to check the file ext of a binary against known library file exts.
		/// FileExt should include the dot e.g. ".dll"
		/// </summary>
		public static bool IsLibraryFileExtension(string FileExt)
		{
			if (FileExt.Equals(".dll", StringComparison.InvariantCultureIgnoreCase)) return true;
			if (FileExt.Equals(".lib", StringComparison.InvariantCultureIgnoreCase)) return true;
			if (FileExt.Equals(".ocx", StringComparison.InvariantCultureIgnoreCase)) return true;
			if (FileExt.Equals(".a", StringComparison.InvariantCultureIgnoreCase)) return true;
			if (FileExt.Equals(".so", StringComparison.InvariantCultureIgnoreCase)) return true;
			if (FileExt.Equals(".dylib", StringComparison.InvariantCultureIgnoreCase)) return true;

			return false;
		}

		/// <summary>
		/// Helper to check the properties of a project and determine whether it can be built in VS.
		/// </summary>
		public static bool IsProjectBuildable(Project Project)
		{
			ThreadHelper.ThrowIfNotOnUIThread();
			return Project.Kind == GuidList.VCSharpProjectKindGuidString || Project.Kind == GuidList.VCProjectKindGuidString;
		}

		/// Helper function to get the full list of all projects in the DTE Solution
		/// Recurses into items because these are actually in a tree structure
		public static Project[] GetAllProjectsFromDTE()
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			try
			{
				List<Project> Projects = new List<Project>();

				foreach (Project Project in UnrealVSPackage.Instance.DTE.Solution.Projects)
				{
					Projects.Add(Project);

					if (Project.ProjectItems != null)
					{
						foreach (ProjectItem Item in Project.ProjectItems)
						{
							GetSubProjectsOfProjectItem(Item, Projects);
						}
					}
				}

				return Projects.ToArray();
			}
			catch (Exception ex)
			{
				Exception AppEx = new ApplicationException("GetAllProjectsFromDTE() failed", ex);
				Logging.WriteLine(AppEx.ToString());
				throw AppEx;
			}
		}

		public static void ExecuteProjectBuild(Project Project,
												string SolutionConfig,
												string SolutionPlatform,
												BatchBuilderToolControl.BuildJob.BuildJobType BuildType,
												Action ExecutingDelegate,
												Action FailedToStartDelegate)
		{
			ThreadHelper.ThrowIfNotOnUIThread();
			IVsHierarchy ProjHierarchy = Utils.ProjectToHierarchyObject(Project);

			if (ProjHierarchy != null)
			{
				SolutionConfigurations SolutionConfigs =
					UnrealVSPackage.Instance.DTE.Solution.SolutionBuild.SolutionConfigurations;

				var MatchedSolutionConfig =
					(from SolutionConfiguration2 Sc in SolutionConfigs select Sc).FirstOrDefault(
						Sc => {
							ThreadHelper.ThrowIfNotOnUIThread();
							return String.CompareOrdinal(Sc.Name, SolutionConfig) == 0 && String.CompareOrdinal(Sc.PlatformName, SolutionPlatform) == 0;
						});

				if (MatchedSolutionConfig != null)
				{
					SolutionContext ProjectSolutionCtxt = MatchedSolutionConfig.SolutionContexts.Item(Project.UniqueName);

					if (ProjectSolutionCtxt != null)
					{
						IVsCfgProvider2 CfgProvider2 = Utils.HierarchyObjectToCfgProvider(ProjHierarchy);
						if (CfgProvider2 != null)
						{
							CfgProvider2.GetCfgOfName(ProjectSolutionCtxt.ConfigurationName, ProjectSolutionCtxt.PlatformName, out IVsCfg Cfg);

							if (Cfg != null)
							{
								ExecutingDelegate?.Invoke();

								int JobResult = VSConstants.E_FAIL;

								if (BuildType == BatchBuilderToolControl.BuildJob.BuildJobType.Build)
								{
									JobResult =
										UnrealVSPackage.Instance.SolutionBuildManager.StartUpdateSpecificProjectConfigurations(
											1,
											new[] { ProjHierarchy },
											new[] { Cfg },
											null,
											new uint[] { 0 },
											null,
											(uint)VSSOLNBUILDUPDATEFLAGS.SBF_OPERATION_BUILD,
											0);
								}
								else if (BuildType == BatchBuilderToolControl.BuildJob.BuildJobType.Rebuild)
								{
									JobResult =
										UnrealVSPackage.Instance.SolutionBuildManager.StartUpdateSpecificProjectConfigurations(
											1,
											new[] { ProjHierarchy },
											new[] { Cfg },
											new uint[] { 0 },
											null,
											null,
											(uint)(VSSOLNBUILDUPDATEFLAGS.SBF_OPERATION_BUILD | VSSOLNBUILDUPDATEFLAGS.SBF_OPERATION_FORCE_UPDATE),
											0);
								}
								else if (BuildType == BatchBuilderToolControl.BuildJob.BuildJobType.Clean)
								{
									JobResult =
										UnrealVSPackage.Instance.SolutionBuildManager.StartUpdateSpecificProjectConfigurations(
											1,
											new[] { ProjHierarchy },
											new[] { Cfg },
											new uint[] { 0 },
											null,
											null,
											(uint)VSSOLNBUILDUPDATEFLAGS.SBF_OPERATION_CLEAN,
											0);
								}

								if (JobResult == VSConstants.S_OK)
								{
									// Job running - show output
									PrepareOutputPane();
								}
								else
								{
									FailedToStartDelegate?.Invoke();
								}
							}
						}
					}
				}
			}
		}


		private static bool LoadConfigFromUBT(Project SelectedProject)
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			string ProjectPath = Path.GetDirectoryName(SelectedProject.FullName);
			string ConfigFileName = Path.Combine(ProjectPath, "UnrealVS.xml");

			// only try to load the xml configuration once
			if (CachedUBTConfigFileName != ConfigFileName)
			{
				CachedUBTConfigFileName = ConfigFileName;
				CachedUBTConfigXml = null;

				try
				{
					XmlDocument ConfigXml = new XmlDocument();
					ConfigXml.Load(ConfigFileName);

					CachedUBTConfigXml = ConfigXml.SelectSingleNode("UnrealVS");

				}
				catch (Exception)
				{
				}
			}

			return (CachedUBTConfigXml != null);
		}


		public static List<string> GetExtraDebuggerCommandArguments(string PlatformName, Project SelectedProject)
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			List<string> Result = new List<string>();

			if (LoadConfigFromUBT(SelectedProject))
			{
				XmlNode PlatformNode = CachedUBTConfigXml.SelectSingleNode(PlatformName);
				if (PlatformNode != null)
				{
					foreach (XmlNode ChildNode in PlatformNode.ChildNodes)
					{
						if (string.Equals(ChildNode.Name, "DebuggerName", StringComparison.CurrentCultureIgnoreCase))
						{
							Result.Add(ChildNode.InnerText);
						}
					}
				}
			}

			return Result;
		}

		public static bool IsGameProject(Project Project)
		{
			ThreadHelper.ThrowIfNotOnUIThread();
			return GetUProjects().ContainsKey(Project.Name);
		}

		public static bool IsTestTargetProject(Project Project)
		{
			ThreadHelper.ThrowIfNotOnUIThread();
			if (Project.Globals.VariableExists["IsTestTarget"])
			{
				return Convert.ToBoolean(Project.Globals["IsTestTarget"]);
			}
			return false;
		}

		/// <summary>
		/// Does the config build something that takes a .uproject on the command line?
		/// </summary>
		public static bool HasUProjectCommandLineArg(string Config)
		{
			return Config.EndsWith("Editor", StringComparison.InvariantCultureIgnoreCase);
		}

		public static string GetUProjectFileName(Project Project)
		{
			ThreadHelper.ThrowIfNotOnUIThread();
			return Project.Name + "." + UProjectExtension;
		}

		public static string GetAutoUProjectCommandLinePrefix(Project Project)
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			var UProjectFileName = GetUProjectFileName(Project);
			var AllUProjects = GetUProjects();

			if (!AllUProjects.TryGetValue(Project.Name, out string UProjectPath))
			{
				// Search the project folder
				var ProjectFolder = Path.GetDirectoryName(Project.FullName);
				var UProjUnderProject = Directory.GetFiles(ProjectFolder, UProjectFileName, SearchOption.TopDirectoryOnly);
				if (UProjUnderProject.Length == 1)
				{
					UProjectPath = UProjUnderProject[0];
				}
			}

			return '\"' + UProjectPath + '\"';
		}

		public static void AddProjects(DirectoryInfo ProjectDir, List<FileInfo> Files)
		{
			Files.AddRange(ProjectDir.EnumerateFiles("*.uproject"));
		}

		/// <summary>
		/// Enumerate projects under the given directory
		/// </summary>
		/// <param name="SolutionDir">Base directory to enumerate</param>
		/// <returns>List of project files</returns>
		static List<FileInfo> EnumerateProjects(DirectoryInfo SolutionDir)
		{
			// Enumerate all the projects in the same directory as the solution. If there's one here, we don't need to consider any other.
			List<FileInfo> ProjectFiles = new List<FileInfo>(SolutionDir.EnumerateFiles("*.uproject"));
			if (ProjectFiles.Count == 0)
			{
				// Build a list of all the parent directories for projects. This includes the UE root, plus any directories referenced via .uprojectdirs files.
				List<DirectoryInfo> ParentProjectDirs = new List<DirectoryInfo>
				{
					SolutionDir
				};

				// Read all the .uprojectdirs files
				foreach (FileInfo ProjectDirsFile in SolutionDir.EnumerateFiles("*.uprojectdirs"))
				{
					foreach (string Line in File.ReadAllLines(ProjectDirsFile.FullName))
					{
						string TrimLine = Line.Trim().Replace(Path.AltDirectorySeparatorChar, Path.DirectorySeparatorChar).Trim(Path.DirectorySeparatorChar);
						if (TrimLine.Length > 0 && !TrimLine.StartsWith(";"))
						{
							try
							{
								ParentProjectDirs.Add(new DirectoryInfo(Path.Combine(SolutionDir.FullName, TrimLine)));
							}
							catch (Exception Ex)
							{
								Logging.WriteLine(String.Format("EnumerateProjects: Exception trying to resolve project directory '{0}': {1}", TrimLine, Ex.Message));
							}
						}
					}
				}

				// Add projects in any subfolders of the parent directories
				HashSet<string> CheckedParentDirs = new HashSet<string>(StringComparer.InvariantCultureIgnoreCase);
				foreach (DirectoryInfo ParentProjectDir in ParentProjectDirs)
				{
					if (CheckedParentDirs.Add(ParentProjectDir.FullName) && ParentProjectDir.Exists)
					{
						foreach (DirectoryInfo ProjectDir in ParentProjectDir.EnumerateDirectories())
						{
							try
							{
								ProjectFiles.AddRange(ProjectDir.EnumerateFiles("*.uproject"));
							} 
							catch(Exception Ex)
							{
								Logging.WriteLine($"AddingProjects: Exception trying to add projects from directory '{ProjectDir}': {Ex.Message}");
							}
						}
					}
				}
			}
			return ProjectFiles;
		}

		/// <summary>
		/// Returns all the .uprojects found under the solution root folder.
		/// </summary>
		public static IDictionary<string, string> GetUProjects()
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			var Folder = GetSolutionFolder();
			if (string.IsNullOrEmpty(Folder))
			{
				return new Dictionary<string, string>();
			}

			if (Folder != CachedUProjectRootFolder)
			{
				Logging.WriteLine("GetUProjects: recaching uproject paths...");
				DateTime Start = DateTime.Now;

				CachedUProjectRootFolder = Folder;
				CachedUProjectPaths = EnumerateProjects(new DirectoryInfo(Folder)).Select(x => x.FullName);
				CachedUProjects = null;

				TimeSpan TimeTaken = DateTime.Now - Start;
				Logging.WriteLine(string.Format("GetUProjects: EnumerateProjects took {0} sec", TimeTaken.TotalSeconds));

				foreach (string CachedUProjectPath in CachedUProjectPaths)
				{
					Logging.WriteLine(String.Format("GetUProjects: found {0}", CachedUProjectPath));
				}

				Logging.WriteLine("    DONE");
			}

			if (CachedUProjects == null)
			{
				Logging.WriteLine("GetUProjects: recaching uproject names...");

				var ProjectPaths = UnrealVSPackage.Instance.GetLoadedProjectPaths();
				var ProjectNames = (from path in ProjectPaths select Path.GetFileNameWithoutExtension(path)).ToArray();

				var CodeUProjects = from UProjectPath in CachedUProjectPaths
									let ProjectName = Path.GetFileNameWithoutExtension(UProjectPath)
									where ProjectNames.Any(name => string.Compare(name, ProjectName, StringComparison.OrdinalIgnoreCase) == 0)
									select new { Name = ProjectName, FilePath = UProjectPath };

				CachedUProjects = new Dictionary<string, string>();

				foreach (var UProject in CodeUProjects)
				{
					if (!CachedUProjects.ContainsKey(UProject.Name))
					{
						CachedUProjects.Add(UProject.Name, UProject.FilePath);
					}
				}

				Logging.WriteLine("    DONE");
			}

			return CachedUProjects;
		}

		public static void GetSolutionConfigsAndPlatforms(out string[] SolutionConfigs, out string[] SolutionPlatforms)
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			var UniqueConfigs = new List<string>();
			var UniquePlatforms = new List<string>();

			SolutionConfigurations DteSolutionConfigs = UnrealVSPackage.Instance.DTE.Solution.SolutionBuild.SolutionConfigurations;
			foreach (SolutionConfiguration2 SolutionConfig in DteSolutionConfigs)
			{
				if (!UniqueConfigs.Contains(SolutionConfig.Name))
				{
					UniqueConfigs.Add(SolutionConfig.Name);
				}
				if (!UniquePlatforms.Contains(SolutionConfig.PlatformName))
				{
					UniquePlatforms.Add(SolutionConfig.PlatformName);
				}
			}

			SolutionConfigs = UniqueConfigs.ToArray();
			SolutionPlatforms = UniquePlatforms.ToArray();
		}

		public static bool SetActiveSolutionConfiguration(string ConfigName, string PlatformName)
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			SolutionConfigurations DteSolutionConfigs = UnrealVSPackage.Instance.DTE.Solution.SolutionBuild.SolutionConfigurations;
			foreach (SolutionConfiguration2 SolutionConfig in DteSolutionConfigs)
			{
				if (string.Compare(SolutionConfig.Name, ConfigName, StringComparison.Ordinal) == 0
					&& string.Compare(SolutionConfig.PlatformName, PlatformName, StringComparison.Ordinal) == 0)
				{
					SolutionConfig.Activate();
					return true;
				}
			}
			return false;
		}

		public static bool SelectProjectInSolutionExplorer(Project Project)
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			// Build path to the project in solution explorer: "SolutionName\\FolderName\\...\\ProjectName"
			// Use UIHierarchy.GetItem() for direct O(depth) lookup instead of walking the entire tree.
			var PathParts = new List<string> { Project.Name };
			var ParentItem = Project.ParentProjectItem;
			while (ParentItem != null)
			{
				var ParentProject = ParentItem.ContainingProject;
				if (ParentProject == null) break;
				PathParts.Insert(0, ParentProject.Name);
				ParentItem = ParentProject.ParentProjectItem;
			}
			string SolutionName = Path.GetFileNameWithoutExtension(UnrealVSPackage.Instance.DTE.Solution.FullName);
			if (string.IsNullOrEmpty(SolutionName))
			{
				return false;
			}
			PathParts.Insert(0, SolutionName);
			string ItemPath = string.Join("\\", PathParts);

			UIHierarchy SolutionExplorer = UnrealVSPackage.Instance.DTE2.ToolWindows.SolutionExplorer;
			try
			{
				UIHierarchyItem Item = SolutionExplorer.GetItem(ItemPath);
				if (Item != null)
				{
					Item.Select(vsUISelectionType.vsUISelectionTypeSelect);
					return true;
				}
			}
			catch (Exception)
			{
			}

			return false;
		}

		public static void OnProjectListChanged()
		{
			CachedUProjects = null;
		}

		private static void PrepareOutputPane()
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			UnrealVSPackage.Instance.DTE.ExecuteCommand("View.Output");

			var Pane = UnrealVSPackage.Instance.GetOutputPane();
			if (Pane != null)
			{
				// Clear and activate the output pane.
				Pane.Clear();

				// @todo: Activating doesn't seem to really bring the pane to front like we would expect it to.
				Pane.Activate();
			}
		}

		/// Called by GetAllProjectsFromDTE() to list items from the project tree
		private static void GetSubProjectsOfProjectItem(ProjectItem Item, List<Project> Projects)
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			if (Item.SubProject != null)
			{
				Projects.Add(Item.SubProject);

				if (Item.SubProject.ProjectItems != null)
				{
					foreach (ProjectItem SubItem in Item.SubProject.ProjectItems)
					{
						GetSubProjectsOfProjectItem(SubItem, Projects);
					}
				}
			}
			if (Item.ProjectItems != null)
			{
				foreach (ProjectItem SubItem in Item.ProjectItems)
				{
					GetSubProjectsOfProjectItem(SubItem, Projects);
				}
			}
		}

		public static string GetSolutionFolder()
		{
			ThreadHelper.ThrowIfNotOnUIThread();

			if (!UnrealVSPackage.Instance.DTE.Solution.IsOpen)
			{
				return string.Empty;
			}

			return Path.GetDirectoryName(UnrealVSPackage.Instance.SolutionFilepath);
		}

		public static string SolutionTitle
		{
			set
			{
				if (!SolutionTextBlockSearched)
				{
					SolutionTextBlockSearched = true;

					if (Utils.FindChild<System.Windows.DependencyObject>(System.Windows.Application.Current.MainWindow, "PART_SolutionNameTextBlock") is var textBlock)
					{
						SolutionTextBlock = Utils.FindChild<TextBlock>(textBlock, null);
					}
				}

				if (SolutionTextBlock != null)
				{
					SolutionTextBlockText = value;
					if (value != null)
					{
						SolutionTextBlockChanging = true;
						SolutionTextBlock.Text = value;
						SolutionTextBlockChanging = false;

						if (!SolutionTextBlockTracked)
						{
							SolutionTextBlockTracked = true;
							var dp = DependencyPropertyDescriptor.FromProperty(TextBlock.TextProperty, typeof(TextBlock));
							dp.AddValueChanged(SolutionTextBlock, SolutionTextChanged);
						}
					}
					else
					{
						if (SolutionTextBlockTracked)
						{
							SolutionTextBlockTracked = false;
							var dp = DependencyPropertyDescriptor.FromProperty(TextBlock.TextProperty, typeof(TextBlock));
							dp.RemoveValueChanged(SolutionTextBlock, SolutionTextChanged);
						}
						SolutionTextBlock.InvalidateVisual();
					}
				}
			}
		}

		static void SolutionTextChanged(object sender, EventArgs e)
		{
			// Needed to prevent visual studio from changing text back to name from property
			if (!SolutionTextBlockChanging && SolutionTextBlockText != null)
			{
				SolutionTextBlockChanging = true;
				SolutionTextBlock.Text = SolutionTextBlockText;
				SolutionTextBlockChanging = false;
			}
		}

		public static string MainWindowTitle
		{
			get => System.Windows.Application.Current.MainWindow.Title;
			set { System.Windows.Application.Current.MainWindow.Title = value; }
		}

		public static T FindChild<T>(System.Windows.DependencyObject Parent, string ChildName) where T : System.Windows.DependencyObject
		{
			if (Parent == null)
			{
				return null;
			}

			int ChildrenCount = VisualTreeHelper.GetChildrenCount(Parent);
			for (int i = 0; i < ChildrenCount; i++)
			{
				var Child = VisualTreeHelper.GetChild(Parent, i);
				if (ChildName != null)
				{
					// If the child's name is set for search
					var FrameworkElement = Child as System.Windows.FrameworkElement;
					if (FrameworkElement != null)
					{
						if (FrameworkElement.Name == ChildName)
						{
							// if the child's name is of the request name
							return (T)Child;
						}
					}
				}
				else
				{
					if (Child is T TypedChild)
					{
						return TypedChild;
					}
				}

				// recursively drill down the tree
				T FoundChild = FindChild<T>(Child, ChildName);
				if (FoundChild != null)
				{
					return FoundChild;
				}
			}

			return null;
		}

		private static string SolutionTextBlockText;
		private static TextBlock SolutionTextBlock;
		private static bool SolutionTextBlockTracked;
		private static bool SolutionTextBlockChanging;
		private static bool SolutionTextBlockSearched;
		private static string CachedUProjectRootFolder = string.Empty;
		private static IEnumerable<string> CachedUProjectPaths = new string[0];
		private static IDictionary<string, string> CachedUProjects = null;

		private static string CachedUBTConfigFileName = string.Empty;
		private static XmlNode CachedUBTConfigXml = null;
	}
}
