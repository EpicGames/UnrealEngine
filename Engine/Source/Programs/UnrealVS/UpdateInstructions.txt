1. Make your changes
2. Update version number in both VSIX manifest files and the AssemblyInfo file. 
	* AssemblyInfo is shared between projects
	* Resources are also shared
	* UnrealVS.vsct is also shared
	* Manifest file is named source.extension.vsixmanifest and is unique per project
	* UnrealVSPackage.cs for major/minor version update (this shows up in visual studio version info)
3. update ReleaseNotes.txt with the changes
4. Checkout the VSIX in Engine\Extras\UnrealVS\ files and rebuild UnrealVS.2022
5. Update the atomfeed.xml in Engine\Extras\UnrealVS\<version>\atomfeed.xml
	* <updated> with new timestamp
	* <Version> with new version number
6. Checkin all the above in 1 CL

Development Tips:
	* In Visual Studio: Tools -> Options.  Then Environment/Extensions.  Add your path to AtomFeed.xml as an additional extension gallery.
